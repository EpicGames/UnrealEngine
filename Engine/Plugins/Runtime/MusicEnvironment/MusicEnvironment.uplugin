{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Music Environment",
	"Description": "A Project-Wide source of musical information (musically synchronized clocks, events, etc.)",
	"Category": "Audio",
	"CreatedBy": "Epic Games, Inc. - Harmonix GenTech",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "MusicEnvironment",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "MusicEnvironmentEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "MusicEnvironmentTests",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
	]
}