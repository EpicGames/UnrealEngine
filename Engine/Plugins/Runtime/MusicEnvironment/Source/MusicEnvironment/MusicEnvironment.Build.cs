// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MusicEnvironment : ModuleRules
{
	public MusicEnvironment(ReadOnlyTargetRules Target) : base(Target)
	{
        PrivateIncludePathModuleNames.AddRange(
            new string[] {
			}
        );

        PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
                "Engine",
				"TimeManagement",
				"GameplayTags",
				"MovieScene",
			}
		);
	}
}
