// Copyright Epic Games, Inc. All Rights Reserved.

#include "FrameBasedMusicMap.h"
#include "Algo/FindLast.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FrameBasedMusicMap)

DEFINE_LOG_CATEGORY_STATIC(LogMusicMap, Log, All);
#define LOCTEXT_NAMESPACE "FrameBasedMusicMap"

namespace FrameBasedMusicMap
{
	FText GetQuantizationName(int32 Quantization)
	{
		switch (Quantization)
		{
		case ETripletQuantizationTicks::ThirtysecondNoteTriplet:	return LOCTEXT("ThirtySecondNoteTriplet", "1/32 Note Triplet");
		case ETripletQuantizationTicks::SixteenthNoteTriplet:		return LOCTEXT("SixteenthNoteTriplet", "1/16 Note Triplet");
		case ETripletQuantizationTicks::EighthNoteTriplet:			return LOCTEXT("EighthNoteTriplet", "1/8 Note Triplet");
		case ETripletQuantizationTicks::QuarterNoteTriplet:			return LOCTEXT("QuarterNoteTriplet", "1/4 Note Triplet");
		case ETripletQuantizationTicks::HalfNoteTriplet:			return LOCTEXT("HalfNoteTriplet", "1/2 Note Triplet");
		case ETripletQuantizationTicks::WholeNoteTriplet:			return LOCTEXT("WholeNoteTriplet", "Whole Note Triplet");
		
		case EWholeQuantizationTicks::ThirtysecondNote:	return LOCTEXT("ThirtySecondNote", "1/32 Note");
		case EWholeQuantizationTicks::SixteenthNote:	return LOCTEXT("SixteenthNote", "1/16 Note");
		case EWholeQuantizationTicks::EighthNote:		return LOCTEXT("EighthNote", "1/8 Note");
		case EWholeQuantizationTicks::QuarterNote:		return LOCTEXT("QuarterNote", "1/4 Note");
		case EWholeQuantizationTicks::HalfNote:			return LOCTEXT("HalfNote", "1/2 Note");
		case EWholeQuantizationTicks::WholeNote:		return LOCTEXT("WholeNote", "Whole Note");
		case EWholeQuantizationTicks::Bar:				return LOCTEXT("Bar", "Bar");
		case EWholeQuantizationTicks::Beat:				return LOCTEXT("Beat", "Beat");


		case EDottedQuantizationTicks::DottedThirtysecondNote:	return LOCTEXT("DottedThirtySecondNote", "Dotted 1/32 Note");
		case EDottedQuantizationTicks::DottedSixteenthNote:		return LOCTEXT("DottedSixteenthNote", "Dotted 1/16 Note");
		case EDottedQuantizationTicks::DottedEighthNote:		return LOCTEXT("DottedEighthNote", "Dotted 1/8 Note");
		case EDottedQuantizationTicks::DottedQuarterNote:		return LOCTEXT("DottedQuarterNote", "Dotted 1/4 Note");
		case EDottedQuantizationTicks::DottedHalfNote:			return LOCTEXT("DottedHalfNote", "Dotted Half Note");
		case EDottedQuantizationTicks::DottedWholeNote:			return LOCTEXT("DottedWholeNote", "Dotted Whole Note");
		}
		return LOCTEXT("Unknown", "Unrecognized Quantization Factor");
	}

	EQuartzCommandQuantization QuantiazationIntervalToQuartz(int32 Quantization)
	{
		switch (Quantization)
		{
		case ETripletQuantizationTicks::ThirtysecondNoteTriplet:	return EQuartzCommandQuantization::None; // quartz doesn't go this small.
		case ETripletQuantizationTicks::SixteenthNoteTriplet:		return EQuartzCommandQuantization::SixteenthNoteTriplet;
		case ETripletQuantizationTicks::EighthNoteTriplet:			return EQuartzCommandQuantization::EighthNoteTriplet;
		case ETripletQuantizationTicks::QuarterNoteTriplet:			return EQuartzCommandQuantization::QuarterNoteTriplet;
		case ETripletQuantizationTicks::HalfNoteTriplet:			return EQuartzCommandQuantization::HalfNoteTriplet;
		case ETripletQuantizationTicks::WholeNoteTriplet:			return EQuartzCommandQuantization::None; // quartz doesn't support this division.

		case EWholeQuantizationTicks::ThirtysecondNote:	return EQuartzCommandQuantization::ThirtySecondNote;
		case EWholeQuantizationTicks::SixteenthNote:	return EQuartzCommandQuantization::SixteenthNote;
		case EWholeQuantizationTicks::EighthNote:		return EQuartzCommandQuantization::EighthNote;
		case EWholeQuantizationTicks::QuarterNote:		return EQuartzCommandQuantization::QuarterNote;
		case EWholeQuantizationTicks::HalfNote:			return EQuartzCommandQuantization::HalfNote;
		case EWholeQuantizationTicks::WholeNote:		return EQuartzCommandQuantization::WholeNote;
		case EWholeQuantizationTicks::Bar:				return EQuartzCommandQuantization::Bar;
		case EWholeQuantizationTicks::Beat:				return EQuartzCommandQuantization::Beat;


		case EDottedQuantizationTicks::DottedThirtysecondNote:	return EQuartzCommandQuantization::None; // quartz doesn't support this division.
		case EDottedQuantizationTicks::DottedSixteenthNote:		return EQuartzCommandQuantization::DottedSixteenthNote;
		case EDottedQuantizationTicks::DottedEighthNote:		return EQuartzCommandQuantization::DottedEighthNote;
		case EDottedQuantizationTicks::DottedQuarterNote:		return EQuartzCommandQuantization::DottedQuarterNote;
		case EDottedQuantizationTicks::DottedHalfNote:			return EQuartzCommandQuantization::DottedHalfNote;
		case EDottedQuantizationTicks::DottedWholeNote:			return EQuartzCommandQuantization::DottedWholeNote;
		}
		return EQuartzCommandQuantization::None;
	}

	TArray<TSharedPtr<int32>> QuantizationOptionsSource = {
		MakeShared<int32>(EWholeQuantizationTicks::Bar),
		MakeShared<int32>(EWholeQuantizationTicks::Beat),
		MakeShared<int32>(EWholeQuantizationTicks::WholeNote),
		MakeShared<int32>(EWholeQuantizationTicks::HalfNote),
		MakeShared<int32>(EWholeQuantizationTicks::QuarterNote),
		MakeShared<int32>(EWholeQuantizationTicks::EighthNote),
		MakeShared<int32>(EWholeQuantizationTicks::SixteenthNote),
		MakeShared<int32>(EWholeQuantizationTicks::ThirtysecondNote),
		MakeShared<int32>(ETripletQuantizationTicks::WholeNoteTriplet),
		MakeShared<int32>(ETripletQuantizationTicks::HalfNoteTriplet),
		MakeShared<int32>(ETripletQuantizationTicks::QuarterNoteTriplet),
		MakeShared<int32>(ETripletQuantizationTicks::EighthNoteTriplet),
		MakeShared<int32>(ETripletQuantizationTicks::SixteenthNoteTriplet),
		MakeShared<int32>(ETripletQuantizationTicks::ThirtysecondNoteTriplet),
		MakeShared<int32>(EDottedQuantizationTicks::DottedWholeNote),
		MakeShared<int32>(EDottedQuantizationTicks::DottedHalfNote),
		MakeShared<int32>(EDottedQuantizationTicks::DottedQuarterNote),
		MakeShared<int32>(EDottedQuantizationTicks::DottedEighthNote),
		MakeShared<int32>(EDottedQuantizationTicks::DottedSixteenthNote),
		MakeShared<int32>(EDottedQuantizationTicks::DottedThirtysecondNote)
	};
}

void UFrameBasedMusicMap::SetFrameResolution(const FFrameRate& NewFrameResolution)
{
	check(NewFrameResolution.IsValid());

	if (NewFrameResolution == FrameResolution)
	{
		return;
	}

	if (FrameResolution.IsValid() && !IsEmpty() && !IsSimple())
	{
		// must be changing sequence's resolution...
		for (FFrameBasedTimeSignaturePoint& Bar : BarMap)
		{
			Bar.OnFrame = FFrameRate::TransformTime(FFrameTime::FromDecimal(Bar.OnFrame), FrameResolution, NewFrameResolution).AsDecimal();
		}
	}

	FrameResolution = NewFrameResolution;
}

void UFrameBasedMusicMap::Init(float InBpm, const FFrameBasedTimeSignature& InTimeSignature)
{
	check(FrameResolution.IsValid());
	Clear();
	TempoMap.Add(FFrameBasedTempoPoint(InBpm, 0, 0.0f));
	BarMap.Add(FFrameBasedTimeSignaturePoint(InTimeSignature, 0, 0, 0.0f));
}

void UFrameBasedMusicMap::SetInitialTempo(float Bpm)
{
	check(!TempoMap.IsEmpty());
	TempoMap[0].SetBpm(Bpm);
}

float UFrameBasedMusicMap::GetInitialTempo() const
{
	return TempoMap.IsEmpty() ? 0.0f : TempoMap[0].Bpm();
}

void UFrameBasedMusicMap::SetInitialTimeSignature(const FFrameBasedTimeSignature& TimeSignature)
{
	check(!BarMap.IsEmpty());
	BarMap[0].TimeSignature = TimeSignature;
}

FFrameBasedTimeSignature UFrameBasedMusicMap::GetInitialTimeSignature() const
{
	return BarMap.IsEmpty() ? FFrameBasedTimeSignature() : BarMap[0].TimeSignature;
}

double UFrameBasedMusicMap::GetFrame(const FMusicalTime& MusicalTime) const
{
	return FrameResolution.AsFrameTime(GetSeconds(MusicalTime)).AsDecimal();
}

double UFrameBasedMusicMap::GetFrame(int32 AtBar) const
{
	return FrameResolution.AsFrameTime(GetSeconds(AtBar)).AsDecimal();
}

FFrameTime UFrameBasedMusicMap::GetFrameTime(const FMusicalTime& MusicalTime) const
{
	return FrameResolution.AsFrameTime(GetSeconds(MusicalTime));
}

FFrameTime UFrameBasedMusicMap::GetFrameTime(int32 AtBar, float AtBeat) const
{
	return FrameResolution.AsFrameTime(GetSeconds(AtBar, AtBeat));
}

FFrameTime UFrameBasedMusicMap::GetFrameTime(double FractionalBar) const
{
	if (IsEmpty())
	{
		return FFrameTime();
	}

	int32 Bar = FMath::FloorToInt32(FractionalBar);
	float Beat =static_cast<float>(static_cast<double>(GetBeatsInBar(Bar)) * FMath::Fractional(FractionalBar));
	return GetFrameTime(Bar, Beat);	
}

FFrameTime UFrameBasedMusicMap::GetFrameTime(int32 AtTick) const
{
	if (IsEmpty())
	{
		return FFrameTime();
	}

	FMusicalTime MusicalTime = GetMusicalTime(AtTick);
	return GetFrameTime(MusicalTime);
}

double UFrameBasedMusicMap::GetSeconds(const FMusicalTime& MusicalTime) const
{
	if (IsEmpty())
	{
		return 0.0;
	}

	check(MusicalTime.IsValid());

	const FFrameBasedTimeSignaturePoint* TimeSignatureAtFrame = GetTimeSignaturePointForBar(MusicalTime.Bar);

	int32 Tick = TimeSignatureAtFrame->MusicalTimeToTick(MusicalTime);

	const FFrameBasedTempoPoint* TempoAtFrame = GetTempoPointForTick(Tick);

	return TempoAtFrame->TickToSeconds(Tick);
}

double UFrameBasedMusicMap::GetSeconds(int32 AtBar, float AtBeat) const
{
	if (IsEmpty())
	{
		return 0.0;
	}

	const FFrameBasedTimeSignaturePoint* TimeSignatureAtFrame = GetTimeSignaturePointForBar(AtBar);

	int32 Tick = TimeSignatureAtFrame->BarBeatToTick(AtBar, AtBeat);

	const FFrameBasedTempoPoint* TempoAtFrame = GetTempoPointForTick(Tick);

	return TempoAtFrame->TickToSeconds(Tick);
}

FString UFrameBasedMusicMap::MakeBarBeatStringForFrame(double DecimalFrame, FrameBasedMusicMap::EBarBeatFormat BarBeatFormat) const
{
	if (IsEmpty())
	{
		return FString(TEXT("0.0.00"));
	}

	const FFrameBasedTimeSignaturePoint* TimeSignatureAtFrame = GetTimeSignaturePointForFrame(DecimalFrame);
	const FFrameBasedTempoPoint* TempoAtFrame = GetTempoPointForFrame(DecimalFrame);

	return MakeBarBeatString(DecimalFrame, TimeSignatureAtFrame, TempoAtFrame, BarBeatFormat);
}

FString UFrameBasedMusicMap::MakeBarBeatStringForFrameTime(const FFrameTime& Frame, FrameBasedMusicMap::EBarBeatFormat BarBeatFormat) const
{
	return MakeBarBeatStringForFrame(Frame.AsDecimal(), BarBeatFormat);
}

FString UFrameBasedMusicMap::MakeBarBeatStringForMusicalTime(const FMusicalTime& MusicalTime, FrameBasedMusicMap::EBarBeatFormat BarBeatFormat) const
{
	FFrameTime FrameTime = GetFrameTime(MusicalTime);
	return MakeBarBeatStringForFrameTime(FrameTime, BarBeatFormat);
}

FString UFrameBasedMusicMap::MakeBarBeatStringForSeconds(double Seconds, FrameBasedMusicMap::EBarBeatFormat BarBeatFormat) const
{
	return MakeBarBeatStringForFrameTime(FrameResolution.AsFrameTime(Seconds), BarBeatFormat);
}

float UFrameBasedMusicMap::GetTempoForFrame(double DecimalFrame) const
{
	return GetTempoForFrameTime(FFrameTime::FromDecimal(DecimalFrame));
}

float UFrameBasedMusicMap::GetTempoForFrameTime(const FFrameTime& FrameTime) const
{
	return GetTempoForSeconds(FrameResolution.AsSeconds(FrameTime));
}

float UFrameBasedMusicMap::GetTempoForSeconds(double Seconds) const
{
	if (TempoMap.IsEmpty())
	{
		return 0.0f;
	}

	if (Seconds <= 0.0)
	{
		return TempoMap[0].Bpm();
	}

	const FFrameBasedTempoPoint* TempoPoint = GetTempoPointForSeconds(Seconds);

	return TempoPoint->Bpm();
}

FFrameBasedTimeSignature UFrameBasedMusicMap::GetTimeSignatureForFrame(double DecimalFrame) const
{
	const FFrameBasedTimeSignaturePoint* Point = GetTimeSignaturePointForFrame(DecimalFrame);
	return Point ? Point->TimeSignature : FFrameBasedTimeSignature();
}

FFrameBasedTimeSignature UFrameBasedMusicMap::GetTimeSignatureForFrameTime(const FFrameTime& FrameTime) const
{
	const FFrameBasedTimeSignaturePoint* Point = GetTimeSignaturePointForFrameTime(FrameTime);
	return Point ? Point->TimeSignature : FFrameBasedTimeSignature();
}

FFrameBasedTimeSignature UFrameBasedMusicMap::GetTimeSignatureForSeconds(double Seconds) const
{
	const FFrameBasedTimeSignaturePoint* Point = GetTimeSignaturePointForSeconds(Seconds);
	return Point ? Point->TimeSignature : FFrameBasedTimeSignature();
}

int32 UFrameBasedMusicMap::GetBeatsInBar(int32 Bar) const
{
	if (BarMap.IsEmpty())
	{
		return 4;
	}
	
	const FFrameBasedTimeSignaturePoint* TimeSignaturePoint = GetTimeSignaturePointForBar(Bar);
	check(TimeSignaturePoint);
	return TimeSignaturePoint->TimeSignature.Numerator;
}

int32 UFrameBasedMusicMap::GetTicksInBar(int32 Bar) const
{
	if (BarMap.IsEmpty())
	{
		return 4 * MusicalTime::TicksPerQuarterNote;
	}
	
	const FFrameBasedTimeSignaturePoint* TimeSignaturePoint = GetTimeSignaturePointForBar(Bar);
	check(TimeSignaturePoint);
	return TimeSignaturePoint->TicksPerBar();	
}

const FFrameBasedTimeSignaturePoint* UFrameBasedMusicMap::GetTimeSignaturePointForTick(int32 Tick) const
{
	if (BarMap.IsEmpty())
	{
		return nullptr;
	}

	if (Tick <= 0)
	{
		return &BarMap[0];
	}

	const FFrameBasedTimeSignaturePoint* TimeSignatureAtTick = Algo::FindLastByPredicate(BarMap, [=](FFrameBasedTimeSignaturePoint const& Point) { return Tick >= Point.OnTick; });
	return TimeSignatureAtTick ? TimeSignatureAtTick : &BarMap[0];
}

const FFrameBasedTimeSignaturePoint* UFrameBasedMusicMap::GetTimeSignaturePointForBar(int32 Bar) const
{
	if (BarMap.IsEmpty())
	{
		return nullptr;
	}

	if (Bar <= 0)
	{
		return &BarMap[0];
	}

	const FFrameBasedTimeSignaturePoint* TimeSignatureAtBar = Algo::FindLastByPredicate(BarMap, [=](FFrameBasedTimeSignaturePoint const& Point) { return Bar >= Point.OnBar; });
	return TimeSignatureAtBar ? TimeSignatureAtBar : &BarMap[0];
}

const FFrameBasedTimeSignaturePoint* UFrameBasedMusicMap::GetTimeSignaturePointForFrame(double DecimalFrame) const
{
	if (BarMap.IsEmpty())
	{
		return nullptr;
	}

	if (DecimalFrame <= 0.0)
	{
		return &BarMap[0];
	}

	const FFrameBasedTimeSignaturePoint* TimeSignatureAtBar = Algo::FindLastByPredicate(BarMap, [=](FFrameBasedTimeSignaturePoint const& Point) { return DecimalFrame >= Point.OnFrame; });
	return TimeSignatureAtBar ? TimeSignatureAtBar : &BarMap[0];
}

const FFrameBasedTimeSignaturePoint* UFrameBasedMusicMap::GetTimeSignaturePointForFrameTime(const FFrameTime& Frame) const
{
	return GetTimeSignaturePointForFrame(Frame.AsDecimal());
}

const FFrameBasedTimeSignaturePoint* UFrameBasedMusicMap::GetTimeSignaturePointForSeconds(double Seconds) const
{
	if (BarMap.IsEmpty() || TempoMap.IsEmpty())
	{
		return nullptr;
	}

	if (Seconds <= 0.0)
	{
		return &BarMap[0];
	}

	const FFrameBasedTempoPoint* TempoAtTime = GetTempoPointForSeconds(Seconds);
	double Tick = TempoAtTime->SecondsToTick(Seconds);
	return GetTimeSignaturePointForTick(Tick);
}

const FFrameBasedTempoPoint* UFrameBasedMusicMap::GetTempoPointForTick(int32 Tick) const
{
	if (TempoMap.IsEmpty())
	{
		return nullptr;
	}

	if (Tick <= 0)
	{
		return &TempoMap[0];
	}

	const FFrameBasedTempoPoint* TempoAtTick = Algo::FindLastByPredicate(TempoMap, [=](FFrameBasedTempoPoint const& Point) { return Tick >= Point.OnTick; });
	return TempoAtTick ? TempoAtTick : &TempoMap[0];
}

const FFrameBasedTempoPoint* UFrameBasedMusicMap::GetTempoPointForFrame(double DecimalFrame) const
{
	return GetTempoPointForFrameTime(FFrameTime::FromDecimal(DecimalFrame));
}

const FFrameBasedTempoPoint* UFrameBasedMusicMap::GetTempoPointForFrameTime(const FFrameTime& Frame) const
{
	return GetTempoPointForSeconds(FrameResolution.AsSeconds(Frame));
}

const FFrameBasedTempoPoint* UFrameBasedMusicMap::GetTempoPointForSeconds(double Seconds) const
{
	if (TempoMap.IsEmpty())
	{
		return nullptr;
	}

	if (Seconds <= 0.0)
	{
		return &TempoMap[0];
	}

	float Ms = static_cast<float>(Seconds * 1000.0);
	const FFrameBasedTempoPoint* TempoAtTime = Algo::FindLastByPredicate(TempoMap, [=](FFrameBasedTempoPoint const& Point) { return Ms >= Point.OnMs; });
	return TempoAtTime ? TempoAtTime : &TempoMap[0];
}

FFrameTime UFrameBasedMusicMap::RemapFrameTime(FFrameTime SourceFrameTime, const UFrameBasedMusicMap* OldMap) const
{
	if (IsEmpty())
	{
		UE_LOGF(LogMusicMap, Warning, "Unable to remap times, MusicMap is empty");
		return SourceFrameTime;
	}

	if (OldMap == nullptr)
	{
		UE_LOGF(LogMusicMap, Warning, "Unable to remap times, Old MusicMap is null");
		return SourceFrameTime;
	}
	
	if (OldMap->IsEmpty())
	{
		UE_LOGF(LogMusicMap, Warning, "Unable to remap times, Old MusicMap is empty");
		return SourceFrameTime;
	}
	
	float Seconds = OldMap->FrameResolution.AsSeconds(SourceFrameTime);
	const FFrameBasedTempoPoint* TempoPoint = OldMap->GetTempoPointForSeconds(Seconds);
	int32 Tick = TempoPoint->SecondsToTick(Seconds);

	FMusicalTime MusicalTime = OldMap->GetMusicalTime(Tick);
	const FFrameBasedTimeSignaturePoint* TimeSignaturePoint = GetTimeSignaturePointForBar(MusicalTime.Bar);
	if (TimeSignaturePoint->TicksPerBar() != MusicalTime.TicksPerBar)
	{
		// Time Signature mismatch! Scale bar to bar!
		MusicalTime.TickInBar = static_cast<int32>( static_cast<float>(MusicalTime.TickInBar * TimeSignaturePoint->TicksPerBar()) / static_cast<float>(MusicalTime.TicksPerBar));
	}
	MusicalTime.TicksPerBar = TimeSignaturePoint->TicksPerBar();
	MusicalTime.TicksPerBeat = TimeSignaturePoint->TicksPerBeat();
	return GetFrameTime(MusicalTime);
}

FMusicalTime UFrameBasedMusicMap::BarsToMusicalTime(double BarsFromStart) const
{
	FMusicalTime Result;
	Result.Bar = FMath::FloorToInt32(BarsFromStart);
	double FractionalBar = BarsFromStart - static_cast<double>(Result.Bar);

	const FFrameBasedTimeSignaturePoint* TimeSignaturePoint = GetTimeSignaturePointForBar(Result.Bar);
	Result.TicksPerBar = TimeSignaturePoint->TicksPerBar();
	Result.TicksPerBeat = TimeSignaturePoint->TicksPerBeat();
	Result.TickInBar = FMath::Clamp(FMath::RoundToInt32( static_cast<double>(Result.TicksPerBar) * FractionalBar), 0, Result.TicksPerBar - 1);
	return Result;
}

FMusicalTime UFrameBasedMusicMap::Quantize(const FMusicalTime& MusicalTime, int32 QuantizationInterval, EQuantizeDirection Direction) const
{
	int32 QuantizedTick = QuantizeToTick(MusicalTime, QuantizationInterval, Direction);

	const FFrameBasedTimeSignaturePoint* TimeSignaturePoint = GetTimeSignaturePointForTick(QuantizedTick);

	return TimeSignaturePoint->TickToMusicalTime(QuantizedTick);
}

int32 UFrameBasedMusicMap::QuantizeToTick(const FMusicalTime& MusicalTime, int32 QuantizationInterval, EQuantizeDirection Direction /*= EQuantizeDirection::Nearest*/) const
{
	int32 TicksInQuantizationInterval = QuantizationInterval;
	switch (QuantizationInterval)
	{
	case FrameBasedMusicMap::EWholeQuantizationTicks::Bar:
		TicksInQuantizationInterval = MusicalTime.TicksPerBar;
		break;
	case FrameBasedMusicMap::EWholeQuantizationTicks::Beat:
		TicksInQuantizationInterval = MusicalTime.TicksPerBeat;
		break;
	}

	const FFrameBasedTimeSignaturePoint* TimeSignaturePoint = GetTimeSignaturePointForBar(MusicalTime.Bar);

	int32 TotalTicks = TimeSignaturePoint->MusicalTimeToTick(MusicalTime);

	int32 TicksInTimeSignature = TotalTicks - TimeSignaturePoint->OnTick;

	int32 Intervals = 0;
	switch (Direction)
	{
	default:
	case EQuantizeDirection::Nearest:
		Intervals = FMath::RoundToInt32(static_cast<float>(TicksInTimeSignature) / static_cast<float>(TicksInQuantizationInterval));
		break;
	case EQuantizeDirection::Ceiling:
		Intervals = FMath::CeilToInt32(static_cast<float>(TicksInTimeSignature) / static_cast<float>(TicksInQuantizationInterval));
		break;
	case EQuantizeDirection::Floor:
		Intervals = FMath::FloorToInt32(static_cast<float>(TicksInTimeSignature) / static_cast<float>(TicksInQuantizationInterval));
		break;
	}

	int32 QuantizedTicksInTimeSignature = Intervals * TicksInQuantizationInterval;

	int32 FinalTick = TimeSignaturePoint->OnTick + QuantizedTicksInTimeSignature;

	return FinalTick;
}

FMusicalTime UFrameBasedMusicMap::Add(const FMusicalTime& MusicalTime, int32 QuantizationInterval) const
{
	const FFrameBasedTimeSignaturePoint* OriginalTimeSignature = GetTimeSignaturePointForBar(MusicalTime.Bar);
	int32 AbsoluteTick = OriginalTimeSignature->MusicalTimeToTick(MusicalTime);
	if (FMath::Abs(QuantizationInterval) == FrameBasedMusicMap::EWholeQuantizationTicks::Bar)
	{
		AbsoluteTick += (QuantizationInterval < 0) ? MusicalTime.TicksPerBar : -MusicalTime.TicksPerBar;
	}
	else if (FMath::Abs(QuantizationInterval) == FrameBasedMusicMap::EWholeQuantizationTicks::Beat)
	{
		AbsoluteTick += (QuantizationInterval < 0) ? MusicalTime.TicksPerBeat : -MusicalTime.TicksPerBeat;
	}
	else
	{
		AbsoluteTick += QuantizationInterval;
	}
	const FFrameBasedTimeSignaturePoint* NewTimeSignature = GetTimeSignaturePointForTick(AbsoluteTick);
	return NewTimeSignature->TickToMusicalTime(AbsoluteTick);
}

void UFrameBasedMusicMap::AddTimeSignature(int32 AtTick, int32 AtBar, int16 Numerator, int16 Denominator)
{
	if (ensure(!BarMap.IsEmpty() || AtTick == 0))
	{
		if (ensure(AtTick == 0 || BarMap.Last().OnTick < AtTick))
		{
			BarMap.Add(FFrameBasedTimeSignaturePoint({Numerator, Denominator}, AtBar, AtTick, GetFrame(AtBar)));
		}
	}
}

void UFrameBasedMusicMap::AddTempo(int32 AtTick, float AtMs, float Bpm)
{
	if (ensure(TempoMap.IsEmpty() || AtTick >= TempoMap.Last().OnTick))
	{
		float ExpectedMs = TempoMap.IsEmpty() ? 0.0f : static_cast<float>(TempoMap.Last().TickToSeconds(static_cast<double>(AtTick)) * 1000.0);
		TempoMap.Add(FFrameBasedTempoPoint(Bpm, AtTick, ExpectedMs));
	}
}

void UFrameBasedMusicMap::InsertTimeSignature(int32 AtBar, int16 Numerator, int16 Denominator)
{
	FFrameBasedTimeSignature InTimeSig(Numerator, Denominator);
	if (!ensure(!BarMap.IsEmpty()))
	{
		return;
	}
	
	int32 Index = BarMap.FindLastByPredicate([AtBar](FFrameBasedTimeSignaturePoint const& Point) { return AtBar >= Point.OnBar; });
	check(Index != INDEX_NONE);
	FFrameBasedTimeSignaturePoint& Point = BarMap[Index];
	if (Point.TimeSignature == InTimeSig)
	{
		// time signature is already at the desired time signature
		return;
	}
	if (Point.OnBar == AtBar)
	{
		// update the time signature of the point at the given bar
		Point.TimeSignature = InTimeSig;
		int32 PrevIndex = Index - 1;
		if (BarMap.IsValidIndex(PrevIndex) && BarMap[PrevIndex].TimeSignature == InTimeSig)
		{
			BarMap.RemoveAt(Index);
			Index = PrevIndex;
		}
	}
	else
	{
		int32 AtTick =  BarMap[Index].BarToTick(AtBar);
		// insert the new time signature at the given tick and index.
		// Note: The index gets incremented here in order to insert the new point after the one we found
		Index = BarMap.Insert(FFrameBasedTimeSignaturePoint(InTimeSig, AtBar, AtTick, GetFrame(AtBar)), Index + 1);
	}

	// update the successive time signatures to their new ticks based on the time signature change
	for (int32 Idx = Index + 1; Idx < BarMap.Num(); /* Idx is conditionally iterated in body */)
	{
		// only remove the immediately "next" time signature if it's a duplicate
		if (Idx == Index + 1 && BarMap[Idx].TimeSignature == InTimeSig)
		{
			BarMap.RemoveAt(Idx);
			continue;
		}
		
		int32 NewTick = BarMap[Idx - 1].BarToTick(BarMap[Idx].OnBar);
		const FFrameBasedTempoPoint* TempoAtFrame = GetTempoPointForTick(NewTick);
		BarMap[Idx].OnTick = NewTick;
		BarMap[Idx].OnFrame = FrameResolution.AsFrameTime(TempoAtFrame->TickToSeconds(NewTick)).AsDecimal();
		++Idx;
	}
}

void UFrameBasedMusicMap::InsertTempo(int32 AtTick, float InBpm)
{
	if (!ensure(!TempoMap.IsEmpty()))
	{
		return;
	}
	
	int32 TempoIndex = Algo::UpperBoundBy(TempoMap, AtTick, &FFrameBasedTempoPoint::OnTick) - 1;
	check(TempoIndex != INDEX_NONE);
	FFrameBasedTempoPoint& Point = TempoMap[TempoIndex];

	if (FMath::IsNearlyEqual(Point.Bpm(), InBpm, 0.001f))
	{
		// previous tempo point is at the desired tempo, so no need to add tempo change
		return;
	}
	
	if (Point.OnTick == AtTick)
	{
		Point.SetBpm(InBpm);
		int32 PrevIndex = TempoIndex - 1;
		if (TempoMap.IsValidIndex(PrevIndex) && FMath::IsNearlyEqual(TempoMap[PrevIndex].Bpm() , InBpm, 0.001f))
		{
			TempoMap.RemoveAt(TempoIndex);
			TempoIndex = PrevIndex;
		}
	}
	else
	{
		float ExpectedMs = TempoMap[TempoIndex].TickToMs(AtTick);
		TempoIndex = TempoMap.Insert(FFrameBasedTempoPoint(InBpm, AtTick, ExpectedMs), TempoIndex + 1);
	}

	for (int32 Idx = TempoIndex + 1; Idx < TempoMap.Num(); /* Duplicate tempos are removed. Idx is conditionally iterated in body */ )
	{
		if (Idx == TempoIndex + 1 && FMath::IsNearlyEqual(TempoMap[Idx].Bpm(), InBpm))
		{
			UE_LOGF(LogMusicMap, Log, "Removing duplicate tempo change");
			TempoMap.RemoveAt(Idx);
			continue;
		}
		
		TempoMap[Idx].OnMs = TempoMap[Idx - 1].TickToMs(TempoMap[Idx].OnTick);
		++Idx;
	}

	// remap the bar tick frame times based on the new tempo change
	int32 TimeSigIndex = BarMap.IndexOfByPredicate([AtTick](const FFrameBasedTimeSignaturePoint& Point) { return AtTick >= Point.OnTick; });
	for (int32 Idx = TimeSigIndex + 1; Idx < BarMap.Num(); ++Idx)
	{
		int32 BarTick = BarMap[Idx].OnTick;
		const FFrameBasedTempoPoint* TempoAtFrame = GetTempoPointForTick(BarTick);
		BarMap[Idx].OnFrame = FrameResolution.AsFrameTime(TempoAtFrame->TickToSeconds(BarTick)).AsDecimal();
	}
}

void UFrameBasedMusicMap::InsertTempo(FMusicalTime AtMusicalTime, float Bpm)
{
	if (const FFrameBasedTimeSignaturePoint* TimeSigPoint = GetTimeSignaturePointForBar(AtMusicalTime.Bar))
	{
		int32 Tick = TimeSigPoint->MusicalTimeToTick(AtMusicalTime);
		InsertTempo(Tick, Bpm);
	}
}

void UFrameBasedMusicMap::InsertTimeSignature(FMusicalTime AtMusicalTime, const FFrameBasedTimeSignature& TimeSignature)
{
	InsertTimeSignature(AtMusicalTime.Bar, TimeSignature.Numerator, TimeSignature.Denominator);
}

bool UFrameBasedMusicMap::RemoveTimeSignature(int32 AtBar)
{
	int32 Index = Algo::BinarySearchBy(BarMap, AtBar, &FFrameBasedTimeSignaturePoint::OnBar);
	
	if (Index == INDEX_NONE)
	{
		return false;
	}

	// don't remove the first and only time signature change
	if (Index == 0)
	{
		if (BarMap.Num() == 1)
		{
			return false;
		}
		// instead of removing the first time signature
		// change it to the successive time signature
		// and remove that one instead
		BarMap[0].TimeSignature = BarMap[1].TimeSignature;
		Index = Index + 1;
	}
	
	BarMap.RemoveAt(Index);

	int32 PrevIndex = Index - 1;
	if (BarMap.IsValidIndex(Index) && BarMap.IsValidIndex(PrevIndex))
	{
		if (BarMap[PrevIndex].TimeSignature == BarMap[Index].TimeSignature)
		{
			BarMap.RemoveAt(Index);
		}
	}

	// update the successive time signatures to their new ticks based on the time signature change
	for (int32 Idx = Index; Idx < BarMap.Num(); ++Idx)
	{
		int32 NewTick = BarMap[Idx - 1].BarToTick(BarMap[Idx].OnBar);
		const FFrameBasedTempoPoint* TempoAtFrame = GetTempoPointForTick(NewTick);
		BarMap[Idx].OnTick = NewTick;
		BarMap[Idx].OnFrame = FrameResolution.AsFrameTime(TempoAtFrame->TickToSeconds(NewTick)).AsDecimal();
	}
	
	return true;
}

bool UFrameBasedMusicMap::RemoveTempo(int32 AtTick)
{
	int32 Index = Algo::BinarySearchBy(TempoMap, AtTick, &FFrameBasedTempoPoint::OnTick);

	if (Index == INDEX_NONE)
	{
		return false;
	}

	if (Index == 0)
	{
		if (TempoMap.Num() == 1)
		{
			return false;
		}
		
		// when removing the first tempo change,
		// 
		TempoMap[0].MicrosecondsPerQuarterNote = TempoMap[1].MicrosecondsPerQuarterNote;
		Index = Index + 1;
	}

	TempoMap.RemoveAt(Index);

	int32 PrevIndex = Index - 1;
	if (TempoMap.IsValidIndex(Index) && TempoMap.IsValidIndex(PrevIndex))
	{
		if (TempoMap[PrevIndex].MicrosecondsPerQuarterNote == TempoMap[Index].MicrosecondsPerQuarterNote)
		{
			TempoMap.RemoveAt(Index);
		}
	}
	
	for (int32 Idx = Index; Idx < TempoMap.Num(); ++Idx)
	{
		TempoMap[Idx].OnMs = TempoMap[Idx - 1].TickToMs(TempoMap[Idx].OnTick);
	}

	// remap the bar tick frame times based on the new tempo change
	int32 TimeSigIndex = BarMap.IndexOfByPredicate([AtTick](const FFrameBasedTimeSignaturePoint& Point) { return AtTick >= Point.OnTick; });
	for (int32 Idx = TimeSigIndex + 1; Idx < BarMap.Num(); ++Idx)
	{
		int32 BarTick = BarMap[Idx].OnTick;
		const FFrameBasedTempoPoint* TempoAtFrame = GetTempoPointForTick(BarTick);
		BarMap[Idx].OnFrame = FrameResolution.AsFrameTime(TempoAtFrame->TickToSeconds(BarTick)).AsDecimal();
	}
	
	return true;
}

bool UFrameBasedMusicMap::RemoveTimeSignature(FMusicalTime AtMusicalTime)
{
	if (AtMusicalTime.TickInBar != 0)
	{
		return false;
	}

	return RemoveTimeSignature(AtMusicalTime.Bar);
}

bool UFrameBasedMusicMap::RemoveTempo(FMusicalTime AtMusicalTime)
{
	if (const FFrameBasedTimeSignaturePoint* TimeSigPoint = GetTimeSignaturePointForBar(AtMusicalTime.Bar))
	{
		int32 AtTick = TimeSigPoint->MusicalTimeToTick(AtMusicalTime);
		RemoveTempo(AtTick);
	}
	return false;
}


bool UFrameBasedMusicMap::HasTimeSignatureChange(int32 AtBar)
{
	return INDEX_NONE != Algo::BinarySearchBy(BarMap, AtBar, &FFrameBasedTimeSignaturePoint::OnBar);
}

bool UFrameBasedMusicMap::HasTempoChange(int32 AtTick)
{
	return INDEX_NONE != Algo::BinarySearchBy(TempoMap, AtTick, &FFrameBasedTempoPoint::OnTick);
}

bool UFrameBasedMusicMap::HasTimeSignatureChange(FMusicalTime AtMusicalTime)
{
	if (AtMusicalTime.TickInBar != 0)
	{
		return false;
	}
	return HasTimeSignatureChange(AtMusicalTime.Bar);
}

bool UFrameBasedMusicMap::HasTempoChange(FMusicalTime AtMusicalTime)
{
	if (const FFrameBasedTimeSignaturePoint* TimeSigPoint = GetTimeSignaturePointForBar(AtMusicalTime.Bar))
	{
		int32 AtTick = TimeSigPoint->MusicalTimeToTick(AtMusicalTime);
		return HasTempoChange(AtTick);
	}
	return false;
}

void UFrameBasedMusicMap::GatherSubdivisonsForTimespan(int32 StartTick, int32 EndTick, int32 QuantizationInterval, TArray<FGridInterval>& OutSubdivisions ) const
{
	FMusicalTime StartMusicalTime = Quantize(GetMusicalTime(StartTick), QuantizationInterval, EQuantizeDirection::Ceiling);
	FMusicalTime EndMusicalTime = Quantize(GetMusicalTime(EndTick), QuantizationInterval, EQuantizeDirection::Floor);

	if (StartMusicalTime >= EndMusicalTime)
	{
		return;
	}

	while (StartMusicalTime <= EndMusicalTime)
	{
		float SubdivisionTime = GetSeconds(StartMusicalTime);

		const FFrameBasedTimeSignaturePoint* TimeSignatureAtFrame = GetTimeSignaturePointForBar(StartMusicalTime.Bar);
		int32 Tick = TimeSignatureAtFrame->MusicalTimeToTick(StartMusicalTime);

		FGridInterval GridInterval(SubdivisionTime, Tick, StartMusicalTime);

		OutSubdivisions.Add(GridInterval);
		StartMusicalTime = Add(StartMusicalTime, QuantizationInterval);
	}
}

void UFrameBasedMusicMap::GatherBarsForTimespan(int32 StartBar, int32 EndBar, int32 NumBarsInterval, TArray<FGridInterval>& OutSubdivisions) const
{
	if (StartBar >= EndBar)
	{
		return;
	}

	while (StartBar <= EndBar)
	{
		float BarSeconds = GetSeconds(StartBar);

		const FFrameBasedTimeSignaturePoint* TimeSignatureAtFrame = GetTimeSignaturePointForBar(StartBar);
		int32 Tick = TimeSignatureAtFrame->BarToTick(StartBar);
		FMusicalTime MusicalTime = TimeSignatureAtFrame->TickToMusicalTime(Tick);

		FGridInterval GridInterval(BarSeconds, Tick, MusicalTime);
		OutSubdivisions.Add(GridInterval);

		StartBar = StartBar + NumBarsInterval;
	}
}

FMusicalTime UFrameBasedMusicMap::GetMusicalTime(int32 Tick) const
{
	if (IsEmpty())
	{
		return FMusicalTime();
	}

	const FFrameBasedTimeSignaturePoint* TimeSignatureAtFrame = GetTimeSignaturePointForTick(Tick);
	const FFrameBasedTempoPoint* TempoAtFrame = GetTempoPointForTick(Tick);

	return TimeSignatureAtFrame->TickToMusicalTime(Tick);
}

FMusicalTime UFrameBasedMusicMap::GetMusicalTime(const FFrameTime& FrameTime) const
{
	if (IsEmpty())
	{
		return FMusicalTime();
	}

	const FFrameBasedTimeSignaturePoint* TimeSignatureAtFrame = GetTimeSignaturePointForFrameTime(FrameTime);
	const FFrameBasedTempoPoint* TempoAtFrame = GetTempoPointForFrameTime(FrameTime);

	double Seconds = FrameResolution.AsSeconds(FrameTime);
	float Tick = TempoAtFrame->SecondsToTick(Seconds);
	return TimeSignatureAtFrame->TickToMusicalTime(Tick);
}

FString UFrameBasedMusicMap::MakeBarBeatString(double DecimalFrame, const FFrameBasedTimeSignaturePoint* TimeSignature, const FFrameBasedTempoPoint* Tempo, FrameBasedMusicMap::EBarBeatFormat BarBeatFormat) const
{
	double Seconds = FrameResolution.AsSeconds(FFrameTime::FromDecimal(DecimalFrame));
	float Tick = Tempo->SecondsToTick(Seconds);
	FMusicalTime MusicalTime = TimeSignature->TickToMusicalTime(Tick);
	// for display to users... bars and beats are 1 based!
	MusicalTime.Bar += (BarBeatFormat == FrameBasedMusicMap::EBarBeatFormat::Position ? 1 : 0);
	double BeatAsDouble = FMath::RoundToDouble((static_cast<float>(MusicalTime.FractionalBeatInBar()) + (BarBeatFormat == FrameBasedMusicMap::EBarBeatFormat::Position ? 1.0f : 0.0f)) * 100.0f);
	if (BeatAsDouble >= ((TimeSignature->TimeSignature.Numerator + (BarBeatFormat == FrameBasedMusicMap::EBarBeatFormat::Position ? 1 : 0)) * 100))
	{
		MusicalTime.Bar++;
		BeatAsDouble = BarBeatFormat == FrameBasedMusicMap::EBarBeatFormat::Position ? 100.0 : 0.0;
	}
	return FString::Printf(TEXT("%d.%.2f"), MusicalTime.Bar, BeatAsDouble / 100.0);
}

void UFrameBasedMusicMap::GetGridMetricsMusic(float InViewStartSeconds, float InViewEndSeconds, float PixelsPerSecond, float MinTickPx, float DesiredMajorTickPx, TArray<FGridInterval>& OutMajorIntervals, TArray<FGridInterval>& OutMinorIntervals) const
{
	const FFrameBasedTempoPoint* StartTempoPoint = GetTempoPointForSeconds(InViewStartSeconds);
	int32 StartTick = StartTempoPoint->SecondsToTick(InViewStartSeconds);
	int32 QuantizedStartTick = StartTick;

	FMusicalTime StartMusicalTime = GetMusicalTime(StartTick);
	int32 StartBar = StartMusicalTime.Bar;

	int32 SubdivisionStepTickSize = FrameBasedMusicMap::EWholeQuantizationTicks::ThirtysecondNote;
	int32 BarStepSize = 1;

	const float DesiredMajorTickSeconds = DesiredMajorTickPx / PixelsPerSecond;

	//We find a subdivision size that intersects with our ViewStart and fits our desired size
	bool bFoundFirstGridLine = false;
	while (!bFoundFirstGridLine)
	{
		if (SubdivisionStepTickSize <= FrameBasedMusicMap::EWholeQuantizationTicks::HalfNote)
		{
			QuantizedStartTick = FMath::FloorToDouble((float)StartTick / (float)SubdivisionStepTickSize) * SubdivisionStepTickSize;
			
			StartTempoPoint = GetTempoPointForTick(QuantizedStartTick);
			double QuantizedStartSeconds = StartTempoPoint->TickToSeconds(QuantizedStartTick);

			int32 SteppedTick = QuantizedStartTick + SubdivisionStepTickSize;
			const FFrameBasedTempoPoint* StartSteppedTempoPoint = GetTempoPointForTick(SteppedTick);
			double SteppedSeconds = StartSteppedTempoPoint->TickToSeconds(SteppedTick);

			float StepSizeSeconds = SteppedSeconds - QuantizedStartSeconds;

			//Make sure were not stepping through a downbeat of next bar
			StartMusicalTime = GetMusicalTime(QuantizedStartTick);
			FMusicalTime SteppedMusicalTime = GetMusicalTime(SteppedTick);

			bool bValidStep = false;
			if (StartMusicalTime.Bar == SteppedMusicalTime.Bar || SteppedMusicalTime.TickInBar == 0)
			{
				bValidStep = true;
			}

			if (StepSizeSeconds > DesiredMajorTickSeconds && bValidStep)
			{
				bFoundFirstGridLine = true;

				FGridInterval NonLinearGridInterval = FGridInterval(QuantizedStartSeconds, QuantizedStartTick, GetMusicalTime(QuantizedStartTick));
				OutMajorIntervals.Add(NonLinearGridInterval);
			}
			else
			{
				SubdivisionStepTickSize *= 2;
			}
		}
		else
		{
			int32 QuantizedStartBar = FMath::FloorToDouble((float)StartBar / (float)BarStepSize) * BarStepSize;
			float QuantizedBarStartTime = GetSeconds(QuantizedStartBar);
			const FFrameBasedTempoPoint* StartSteppedTempoPoint = GetTempoPointForSeconds(QuantizedBarStartTime);
			QuantizedStartTick = StartSteppedTempoPoint->SecondsToTick(QuantizedBarStartTime);

			int32 SteppedBar = QuantizedStartBar + BarStepSize;

			float SecondsAtSteppedBar = GetSeconds(SteppedBar);

			float StepSizeSeconds = SecondsAtSteppedBar - QuantizedBarStartTime;

			if (StepSizeSeconds > DesiredMajorTickSeconds || BarStepSize >= 2048)
			{
				bFoundFirstGridLine = true;

				FGridInterval NonLinearGridInterval = FGridInterval(QuantizedBarStartTime, QuantizedStartTick, BarsToMusicalTime(QuantizedStartBar));
				OutMajorIntervals.Add(NonLinearGridInterval);
			}
			else
			{
				BarStepSize *= 2;
			}
		}
	}

	const FFrameBasedTempoPoint* EndTempoPoint = GetTempoPointForSeconds(InViewEndSeconds);
	int32 EndTick = EndTempoPoint->SecondsToTick(InViewEndSeconds);

	int32 StepSizeTicks = 0;

	for (int32 CurrentTick = QuantizedStartTick; CurrentTick < EndTick; CurrentTick += StepSizeTicks)
	{
		ComputeGridSpacingAtTick(
			CurrentTick,
			DesiredMajorTickSeconds,
			StepSizeTicks);

		int32 SteppedTicks = CurrentTick + StepSizeTicks;

		FMusicalTime CurrentMusicalTime = GetMusicalTime(CurrentTick);
		FMusicalTime SteppedMusicalTime = GetMusicalTime(SteppedTicks);

		//After computing spacing, we may need to correct past intervals if the spacing dosen't make sense
		if (SteppedMusicalTime.Bar != CurrentMusicalTime.Bar && CurrentMusicalTime.Bar > 0)
		{
			int32 NumBarsStepped = SteppedMusicalTime.Bar - CurrentMusicalTime.Bar;

			//Make sure our step size dosen't take us into next bar, i.e. always favour marker on downbeat
			if (SteppedMusicalTime.TickInBar > 0)
			{
				OutMajorIntervals.Pop();
				
				CurrentTick = OutMajorIntervals.Last().IntervalTick;
				CurrentMusicalTime = OutMajorIntervals.Last().MusicalTime;

				StepSizeTicks = CurrentMusicalTime.TicksPerBar - CurrentMusicalTime.TickInBar;
				SteppedTicks = CurrentTick + StepSizeTicks;
				SteppedMusicalTime = GetMusicalTime(SteppedTicks);
			}
			//Make sure we aren't stepping up in multiple number of bars at wrong interval
			else if (SteppedMusicalTime.Bar % NumBarsStepped != 0)
			{
				//We need to increase the previous interval 
				FMusicalTime PreviousSteppedMusicalTime = OutMajorIntervals.Last().MusicalTime;

				OutMajorIntervals.Pop();

				if (OutMajorIntervals.Num() > 0)
				{
					FMusicalTime PreviousMusicalTime = OutMajorIntervals.Last().MusicalTime;

					NumBarsStepped = PreviousSteppedMusicalTime.Bar - PreviousMusicalTime.Bar;
					NumBarsStepped *= 2;

					int32 SteppedBars = PreviousMusicalTime.Bar + NumBarsStepped;

					const FFrameBasedTimeSignaturePoint* TimeSigPoint = GetTimeSignaturePointForBar(SteppedBars);
					int32 BarsAtTimeSig = SteppedBars - TimeSigPoint->OnBar;

					StepSizeTicks = (TimeSigPoint->OnTick - CurrentTick) + BarsAtTimeSig * TimeSigPoint->TicksPerBar();
					SteppedTicks = CurrentTick + StepSizeTicks;
					SteppedMusicalTime = GetMusicalTime(SteppedTicks);
				}
			}
		}

		const FFrameBasedTempoPoint* SteppedTempoPoint = GetTempoPointForTick(SteppedTicks);
		float SteppedSizeSeconds = SteppedTempoPoint->TickToMs(SteppedTicks) / 1000.f;

		FGridInterval NonLinearGridInterval = FGridInterval(SteppedSizeSeconds, SteppedTicks, SteppedMusicalTime);
		OutMajorIntervals.Add(NonLinearGridInterval);
	}

	for (int32 Idx = 0; Idx < OutMajorIntervals.Num() - 1; ++Idx)
	{
		ComputeMinorGridSpacing(OutMajorIntervals[Idx].IntervalTick, OutMajorIntervals[Idx + 1].IntervalTick, OutMinorIntervals);
	}
}

void UFrameBasedMusicMap::ComputeGridSpacingAtTick(int32 TickToCompute, float DesiredMajorTickSeconds, int32& OutMajorIntervalTicks) const
{
	const FFrameBasedTempoPoint* TempoPoint = GetTempoPointForTick(TickToCompute);
	float TimeToComputeSeconds = TempoPoint->TickToSeconds(TickToCompute);

	float DesiredTimeStepped = TimeToComputeSeconds + DesiredMajorTickSeconds;

	int32 TickStepped = TempoPoint->MsToTick(DesiredTimeStepped * 1000.f);

	int32 StepTickSize = TickStepped - TickToCompute;

	//This will round up to lowest 2^N tick size that is larger than our desired size
	int32 PowTwo = FMath::RoundUpToPowerOfTwo(StepTickSize / FrameBasedMusicMap::EWholeQuantizationTicks::ThirtysecondNote);
	StepTickSize = FrameBasedMusicMap::EWholeQuantizationTicks::ThirtysecondNote * PowTwo;

	FMusicalTime StartMusicalTime = GetMusicalTime(TickToCompute);
	if (StartMusicalTime.TickInBar == 0 && StepTickSize > FrameBasedMusicMap::EWholeQuantizationTicks::HalfNote)
	{
		FMusicalTime SteppedMusicalTime = GetMusicalTime(TickStepped);

		int32 NumBarsStepped = SteppedMusicalTime.Bar - StartMusicalTime.Bar;
		NumBarsStepped++;

		NumBarsStepped = FMath::Clamp(NumBarsStepped, 1, 2048);
		int32 NumMultipleBarsStepped = FMath::RoundUpToPowerOfTwo(NumBarsStepped);

		int32 SteppedBars = StartMusicalTime.Bar + NumMultipleBarsStepped;

		const FFrameBasedTimeSignaturePoint* TimeSigPoint = GetTimeSignaturePointForBar(SteppedBars);
		int32 BarsAtTimeSig = SteppedBars - TimeSigPoint->OnBar;

		StepTickSize = (TimeSigPoint->OnTick - TickToCompute) + BarsAtTimeSig * TimeSigPoint->TicksPerBar();
	}
	else
	{
		//We can't step upwards if our starting tick isn't on a subdivision boundary of that size
		//I.e. if we are on the tick 3/8th notes into a bar, we can't start stepping in 1/4 notes
		while (StartMusicalTime.TickInBar % StepTickSize != 0 && StepTickSize > FrameBasedMusicMap::EWholeQuantizationTicks::ThirtysecondNote)
		{
			StepTickSize /= 2;
		}

		if (StepTickSize < FrameBasedMusicMap::EWholeQuantizationTicks::ThirtysecondNote)
		{
			StepTickSize = FrameBasedMusicMap::EWholeQuantizationTicks::ThirtysecondNote;
		}
	}

	OutMajorIntervalTicks = StepTickSize;
}

void UFrameBasedMusicMap::ComputeMinorGridSpacing(int32 StartTick, int32 EndTick, TArray<FGridInterval>& OutMinorIntervals) const
{
	int32 StepTickSize = EndTick - StartTick;
    if (StepTickSize <= 0)
    {
    	return;
    }
	FMusicalTime StartMusicalTime = GetMusicalTime(StartTick);
	FMusicalTime EndMusicalTime = GetMusicalTime(EndTick);

	int32 BarStepSize = EndMusicalTime.Bar - StartMusicalTime.Bar;

	if (StepTickSize >= StartMusicalTime.TicksPerBar && BarStepSize >= 4)
	{
		int32 MinorIntervalBars = BarStepSize / 4;
		GatherBarsForTimespan(StartMusicalTime.Bar, EndMusicalTime.Bar, MinorIntervalBars, OutMinorIntervals);
	}
	else
	{
		int32 MinorTickSize = StartMusicalTime.TicksPerBeat;
		while (MinorTickSize >= StepTickSize)
		{
			MinorTickSize /= 2;
		}
		MinorTickSize = FMath::Clamp(MinorTickSize, FrameBasedMusicMap::EWholeQuantizationTicks::ThirtysecondNote, FrameBasedMusicMap::EWholeQuantizationTicks::HalfNote);
		GatherSubdivisonsForTimespan(StartTick, EndTick, MinorTickSize, OutMinorIntervals);
	}
}

TArray<FFrameBasedTempoPoint> UFrameBasedMusicMap::GetTempoMetrics(float InViewStartSeconds, float InViewEndSeconds) const
{
	TArray<FFrameBasedTempoPoint> TempoPoints;

	float ViewStartMs = InViewStartSeconds * 1000.f;

	float ViewEndMs = InViewEndSeconds * 1000.f;

	for (FFrameBasedTempoPoint TempoPoint : TempoMap)
	{
		if (TempoPoint.OnMs > ViewStartMs && TempoPoint.OnMs < ViewEndMs)
		{
			TempoPoints.Add(TempoPoint);
		}
	}

	return TempoPoints;
}

void UFrameBasedMusicMap::GetTempoTimeSignatureMarkers(float InStartSeconds, float InEndSeconds, TArray<FTempoTimeSignatureMarker>& OutMarkers) const
{
	float ViewStartMs = InStartSeconds * 1000.f;
	float ViewEndMs = InEndSeconds * 1000.f;

	const FFrameBasedTempoPoint* StartTempoPoint = GetTempoPointForSeconds(InStartSeconds);
	int32 ViewStartTick = StartTempoPoint->SecondsToTick(InStartSeconds);

	const FFrameBasedTempoPoint* EndTempoPoint = GetTempoPointForSeconds(InEndSeconds);
	int32 ViewEndTick = EndTempoPoint->SecondsToTick(InEndSeconds);

	for (const FFrameBasedTempoPoint& TempoPoint : TempoMap)
	{
		if (TempoPoint.OnMs > ViewStartMs && TempoPoint.OnMs < ViewEndMs)
		{
			FTempoTimeSignatureMarker Marker;
			Marker.Seconds = TempoPoint.OnMs / 1000.f;
			Marker.TempoPoint = TempoPoint;

			OutMarkers.Add(Marker);
		}
	}

	for (const FFrameBasedTimeSignaturePoint& TimeSigPoint : BarMap)
	{
		if (TimeSigPoint.OnTick > ViewStartTick && TimeSigPoint.OnTick < ViewEndTick)
		{
			if (FTempoTimeSignatureMarker* FoundMarker = OutMarkers.FindByPredicate([TimeSigPoint](const FTempoTimeSignatureMarker& Marker) { return Marker.TempoPoint.IsSet() && Marker.TempoPoint->OnTick == TimeSigPoint.OnTick; }))
			{
				FoundMarker->TimeSigPoint = TimeSigPoint;
			}
			else
			{
				FTempoTimeSignatureMarker Marker;
				Marker.TimeSigPoint = TimeSigPoint;

				const FFrameBasedTempoPoint* TempoPoint = GetTempoPointForTick(TimeSigPoint.OnTick);
				Marker.Seconds = TempoPoint->TickToSeconds(TimeSigPoint.OnTick);

				OutMarkers.Add(Marker);
			}
		}
	}
}


#undef LOCTEXT_NAMESPACE
