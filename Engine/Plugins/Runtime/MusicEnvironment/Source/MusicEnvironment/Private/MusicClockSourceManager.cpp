// Copyright Epic Games, Inc. All Rights Reserved.

#include "MusicClockSourceManager.h"

#include "MusicEnvironmentSubsystem.h" 

#include UE_INLINE_GENERATED_CPP_BY_NAME(MusicClockSourceManager)

TScriptInterface<IMusicEnvironmentClockSource> UMusicClockSourceManager::FindClock(const FGameplayTag& Tag, bool bExactMatch)
{
	CleanMap();
	if (const TWeakInterfacePtr<IMusicEnvironmentClockSource>* Clock = TaggedClocks.Find(Tag))
	{
		return Clock->ToScriptInterface();;
	}
	if (!bExactMatch)
	{
		int32 NumberOfMatches = 0;
		TWeakInterfacePtr<IMusicEnvironmentClockSource> MatchingClock;
		FGameplayTag MatchingTag;
		for (auto& It : TaggedClocks)
		{
			if (It.Key.MatchesTag(Tag))
			{
				++NumberOfMatches;
				if (!MatchingClock.IsValid())
				{
					MatchingClock = It.Value;
					MatchingTag = It.Key;
				}
			}
		}
		if (MatchingClock.IsValid())
		{
			if (NumberOfMatches > 1)
			{ 
				UE_LOGF(LogMusicEnvironment, Warning, "FindClock found %d clocks that match the tag '%ls'. Returning clock tagged '%ls'.", NumberOfMatches, *Tag.ToString(), *MatchingTag.ToString());
			}
			return MatchingClock.ToScriptInterface();
		}
	}
	return nullptr;
}

void UMusicClockSourceManager::AddTaggedClock(const FGameplayTag& Tag, TScriptInterface<IMusicEnvironmentClockSource> InClock)
{
	UObject* AsObject = InClock.GetObject();
	AddTaggedClock(Tag, AsObject);
}

void UMusicClockSourceManager::AddTaggedClock(const FGameplayTag& Tag, UObject* InClock)
{
	CleanMap();
	TWeakInterfacePtr<IMusicEnvironmentClockSource> WeakInterfacePtr(InClock);
	if (!WeakInterfacePtr.IsValid())
	{
		return;
	}
	if (TaggedClocks.Contains(Tag))
	{
		if (WeakInterfacePtr != TaggedClocks[Tag])
		{
			UE_LOGF(LogMusicEnvironment, Warning, "The MusicClockSourceManager is already tracking a clock tagged '%ls'. It will be replaced by this new request.", *Tag.ToString());
			TaggedClocks[Tag] = WeakInterfacePtr;
		}
	}
	else
	{
		TaggedClocks.Add(Tag, WeakInterfacePtr);
	}
}

void UMusicClockSourceManager::RemoveTaggedClock(TScriptInterface<IMusicEnvironmentClockSource> InClock)
{
	UObject* AsObject = InClock.GetObject();
	RemoveTaggedClock(AsObject);
}

void UMusicClockSourceManager::RemoveTaggedClock(UObject* InClock)
{
	CleanMap();
	TWeakInterfacePtr<IMusicEnvironmentClockSource> WeakInterfacePtr(InClock);
	if (WeakInterfacePtr.IsValid())
	{
		TaggedClocks = TaggedClocks.FilterByPredicate([&](const TPair<FGameplayTag, TWeakInterfacePtr<IMusicEnvironmentClockSource>>& ClockEntry) { return ClockEntry.Value != WeakInterfacePtr; });
	}
}

void UMusicClockSourceManager::RemoveClockWithTag(const FGameplayTag& Tag)
{
	CleanMap();
	TaggedClocks.Remove(Tag);
}

TScriptInterface<IMusicEnvironmentClockSource> UMusicClockSourceManager::GetGlobalMusicClockAuthority()
{
	CleanStack();
	if (GlobalClockSourceStack.IsEmpty())
	{
		return nullptr;
	}
	return GlobalClockSourceStack.Last().ToScriptInterface();
}

bool UMusicClockSourceManager::PushGlobalMusicClockAuthority(TScriptInterface<IMusicEnvironmentClockSource> InClock)
{
	UObject* AsObject = InClock.GetObject();
	return PushGlobalMusicClockAuthority(AsObject);
}

bool UMusicClockSourceManager::PushGlobalMusicClockAuthority(UObject* InClock)
{
	CleanStack();
	TWeakInterfacePtr<IMusicEnvironmentClockSource> WeakInterfacePtr(InClock);
	if (!WeakInterfacePtr.IsValid())
	{
		return false;
	}
	GlobalClockSourceStack.Add(WeakInterfacePtr);
	return true;
}

void UMusicClockSourceManager::RemoveGlobalClockAuthority(TScriptInterface<IMusicEnvironmentClockSource> InClock)
{
	UObject* AsObject = InClock.GetObject();
	RemoveGlobalClockAuthority(AsObject);
}

void UMusicClockSourceManager::RemoveGlobalClockAuthority(UObject* InClock)
{
	CleanStack();
	TWeakInterfacePtr<IMusicEnvironmentClockSource> WeakInterfacePtr(InClock);
	if (!WeakInterfacePtr.IsValid())
	{
		return;
	}
	GlobalClockSourceStack.RemoveAll([&](const TWeakInterfacePtr<IMusicEnvironmentClockSource>& ClockEntry) { return ClockEntry == WeakInterfacePtr; });
}

void UMusicClockSourceManager::PopMusicClockAuthority()
{
	CleanStack();
	if (!GlobalClockSourceStack.IsEmpty())
	{
		GlobalClockSourceStack.Pop(EAllowShrinking::No);
	}
}

void UMusicClockSourceManager::CleanMap()
{
	TaggedClocks = TaggedClocks.FilterByPredicate([](const TPair<FGameplayTag, TWeakInterfacePtr<IMusicEnvironmentClockSource>>& ClockEntry){ return ClockEntry.Value.IsValid(); });
}

void UMusicClockSourceManager::CleanStack()
{
	GlobalClockSourceStack.RemoveAll([](const TWeakInterfacePtr<IMusicEnvironmentClockSource>& ClockEntry){ return !ClockEntry.IsValid(); });
}
