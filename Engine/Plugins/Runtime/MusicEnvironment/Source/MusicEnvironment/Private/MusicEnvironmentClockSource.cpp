// Copyright Epic Games, Inc. All Rights Reserved.

#include "MusicEnvironmentClockSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MusicEnvironmentClockSource)

void IMusicEnvironmentClockSource::GetCurrentBarBeat(float& Bar, float& BeatInBar)
{
	FMusicalTime MusicalTime = GetPositionMusicalTime();
	Bar = MusicalTime.FractionalBar();
	BeatInBar = MusicalTime.FractionalBeatInBar();
}
