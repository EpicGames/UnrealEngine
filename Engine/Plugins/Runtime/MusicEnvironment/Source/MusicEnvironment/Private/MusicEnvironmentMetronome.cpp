// Copyright Epic Games, Inc. All Rights Reserved.

#include "MusicEnvironmentMetronome.h"

// Add default functionality here for any IMovieSceneMetronome functions that are not pure virtual.

#include UE_INLINE_GENERATED_CPP_BY_NAME(MusicEnvironmentMetronome)
void IMusicEnvironmentMetronome::SetMusicMap(UFrameBasedMusicMap* InMusicMap)
{
	MusicMap = TStrongObjectPtr<UFrameBasedMusicMap>(InMusicMap);
	OnMusicMapSet();
}

bool IMusicEnvironmentMetronome::SetTempo(const float Bpm)
{
	if (MusicMap.IsValid() && !MusicMap->IsSimple())
	{
		return false;
	}
	return OnSetTempo(Bpm);
}

void IMusicEnvironmentMetronome::SetSpeed(float Speed)
{
	OnSetSpeed(Speed);
}

void IMusicEnvironmentMetronome::SetVolume(float InVolume)
{
	OnSetVolume(InVolume);
}

void IMusicEnvironmentMetronome::SetMuted(bool bInMuted)
{
	OnSetMuted(bInMuted);
}



