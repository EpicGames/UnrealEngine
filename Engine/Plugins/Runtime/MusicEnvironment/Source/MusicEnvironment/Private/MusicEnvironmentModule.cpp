// Copyright Epic Games, Inc. All Rights Reserved.

#include "MusicEnvironmentModule.h"

#include "Logging/MessageLog.h"

IMPLEMENT_MODULE(FMusicEnvironmentModule, MusicEnvironment);
