// Copyright Epic Games, Inc. All Rights Reserved.

#include "MusicEnvironmentSubsystem.h"

#include "MusicClockSourceManager.h"
#include "MusicEnvironmentMetronome.h"

#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MusicEnvironmentSubsystem)

DEFINE_LOG_CATEGORY(LogMusicEnvironment);

UMusicEnvironmentSubsystem& UMusicEnvironmentSubsystem::Get()
{
	return *GEngine->GetEngineSubsystem< UMusicEnvironmentSubsystem>();
}

UMusicClockSourceManager* UMusicEnvironmentSubsystem::GetClockSourceManager()
{
	if (!ClockSourceManager)
	{
		ClockSourceManager = NewObject<UMusicClockSourceManager>(this);
	}
	return ClockSourceManager;
}

bool UMusicEnvironmentSubsystem::SetMetronomeClass(const UClass* InMetronomeType)
{
	if (!InMetronomeType->ImplementsInterface(UMusicEnvironmentMetronome::StaticClass()))
	{
		return false;
	}
	MetronomeType = InMetronomeType;
	return true;
}

bool UMusicEnvironmentSubsystem::CanSpawnMetronome() const
{
	return MetronomeType.IsValid();
}

TScriptInterface<IMusicEnvironmentMetronome> UMusicEnvironmentSubsystem::SpawnMetronome(UObject* Outer, FName Name)
{
	if (!MetronomeType.IsValid())
	{
		return nullptr;
	}
	return TScriptInterface<IMusicEnvironmentMetronome>(NewObject<UObject>(Outer, MetronomeType.Get(), MakeUniqueObjectName(Outer, MetronomeType.Get(), Name), RF_Transient));
}
