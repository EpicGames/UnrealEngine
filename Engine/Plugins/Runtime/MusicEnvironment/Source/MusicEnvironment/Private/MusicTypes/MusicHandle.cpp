// Copyright Epic Games, Inc. All Rights Reserved.

#include "MusicTypes/MusicHandle.h"
#include "MusicClockSourceManager.h"
#include "MusicEnvironmentSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MusicHandle)

void IMusicHandle::Stop()
{
	RelinquishGlobalMusicClockAuthority();
	UnregisterAsTaggedClock();
	Stop_Internal();
}

void IMusicHandle::Kill()
{
	RelinquishGlobalMusicClockAuthority();
	UnregisterAsTaggedClock();
	Kill_Internal();
}

void IMusicHandle::BranchOnTransportState(EMusicHanldeTransportState& Branches) const
{
	Branches = GetTransportState();
}

void IMusicHandle::GetCurrentBarBeat(float& Bar, float& BeatInBar, EMusicHanldeClockValidity& Branches)
{
	TScriptInterface<IMusicEnvironmentClockSource> MusicClockSource = GetMusicClockSource();
	if (MusicClockSource)
	{
		MusicClockSource->GetCurrentBarBeat(Bar, BeatInBar);
		Branches = EMusicHanldeClockValidity::ClockValid;
	}
	else
	{
		Bar = 0.0f;
		BeatInBar = 0.0f;
		Branches = EMusicHanldeClockValidity::ClockInvalid;
	}
}

void IMusicHandle::BecomeGlobalMusicClockAuthority()
{
	UMusicEnvironmentSubsystem::Get().GetClockSourceManager()->PushGlobalMusicClockAuthority(GetMusicClockSource());
	IsOnGlobalClockAuthorityStack = true;
}

void IMusicHandle::RelinquishGlobalMusicClockAuthority()
{
	if (IsOnGlobalClockAuthorityStack)
	{
		UMusicEnvironmentSubsystem::Get().GetClockSourceManager()->RemoveGlobalClockAuthority(GetMusicClockSource());
		IsOnGlobalClockAuthorityStack = false;
	}
}

void IMusicHandle::RegisterAsTaggedClock(const FGameplayTag& RegisterAsTagged)
{
	if (RegisterAsTagged.IsValid())
	{
		UMusicEnvironmentSubsystem::Get().GetClockSourceManager()->AddTaggedClock(RegisterAsTagged, GetMusicClockSource());
		RegisteredToTags.AddTag(RegisterAsTagged);
	}
}

void IMusicHandle::UnregisterAsTaggedClock(const FGameplayTag& RegisterAsTagged)
{
	if (!RegisterAsTagged.IsValid())
	{
		// remove all registrations...
		UMusicEnvironmentSubsystem::Get().GetClockSourceManager()->RemoveTaggedClock(GetMusicClockSource());
		RegisteredToTags.Reset();
	}
	else if (RegisteredToTags.HasTag(RegisterAsTagged))
	{
		UMusicEnvironmentSubsystem::Get().GetClockSourceManager()->RemoveClockWithTag(RegisterAsTagged);
		RegisteredToTags.RemoveTag(RegisterAsTagged);
	}
}

void UMusicHandleBlueprintHelpers::BranchOnTransportState(TScriptInterface<IMusicHandle> Handle, EMusicHanldeTransportState& Branches)
{
	Branches = UMusicHandleBlueprintHelpers::GetTransportState(Handle);
}

EMusicHanldeTransportState UMusicHandleBlueprintHelpers::GetTransportState(TScriptInterface<IMusicHandle> Handle)
{
	if (!Handle)
	{
		return EMusicHanldeTransportState::Stale;
	}
	return Handle->GetTransportState();
}
