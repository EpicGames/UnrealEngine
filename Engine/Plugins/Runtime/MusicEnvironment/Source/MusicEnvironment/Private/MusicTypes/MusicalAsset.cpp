// Copyright Epic Games, Inc. All Rights Reserved.

#include "MusicTypes/MusicalAsset.h"
#include "MusicTypes/MusicHandle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MusicalAsset)


TScriptInterface<IMusicHandle> IMusicalAsset::PrepareToPlay(UObject* PlaybackContext, UAudioComponent* OnComponent,
	float FromSeconds, bool BecomeAuthoritativeClock, const FGameplayTag RegisterAsTaggedClock, bool IsAudition)
{
	TScriptInterface<IMusicHandle> Handle = PrepareToPlay_Internal(PlaybackContext, OnComponent, FromSeconds, IsAudition);
	if (Handle)
	{
		if (BecomeAuthoritativeClock)
		{
			Handle->BecomeGlobalMusicClockAuthority();
		}

		if (RegisterAsTaggedClock.IsValid())
		{
			Handle->RegisterAsTaggedClock(RegisterAsTaggedClock);
		}
	}
	return Handle;
}

TScriptInterface<IMusicHandle> IMusicalAsset::Play(UObject* PlaybackContext, UAudioComponent* OnComponent,
	float FromSeconds, bool BecomeAuthoritativeClock, const FGameplayTag RegisterAsTaggedClock, bool IsAudition)
{
	TScriptInterface<IMusicHandle> Handle = Play_Internal(PlaybackContext, OnComponent, FromSeconds, IsAudition);
	if (Handle)
	{
		if (BecomeAuthoritativeClock)
		{
			Handle->BecomeGlobalMusicClockAuthority();
		}

		if (RegisterAsTaggedClock.IsValid())
		{
			Handle->RegisterAsTaggedClock(RegisterAsTaggedClock);
		}
	}
	return Handle;
}
