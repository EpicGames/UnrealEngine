// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/FrameRate.h"
#include "Misc/MusicalTime.h"
#include "UObject/Interface.h"
#include "Sound/QuartzQuantizationUtilities.h"

#include "FrameBasedMusicMap.generated.h"

#define UE_API MUSICENVIRONMENT_API

struct FFrameTime;

namespace FrameBasedMusicMap
{
	enum ETripletQuantizationTicks : int32
	{
		ThirtysecondNoteTriplet = (MusicalTime::TicksPerQuarterNote / 4) / 3,	// 80
		SixteenthNoteTriplet    = (MusicalTime::TicksPerQuarterNote / 2) / 3,	// 160
		EighthNoteTriplet       = (MusicalTime::TicksPerQuarterNote) / 3,		// 320
		QuarterNoteTriplet      = (MusicalTime::TicksPerQuarterNote * 2) / 3,	// 640
		HalfNoteTriplet         = (MusicalTime::TicksPerQuarterNote * 4) / 3,	// 1280
		WholeNoteTriplet        = (MusicalTime::TicksPerQuarterNote * 8) / 3,	// 2560
	};

	enum EWholeQuantizationTicks : int32
	{
		ThirtysecondNote        = MusicalTime::TicksPerQuarterNote / 8,			// 120
		SixteenthNote           = MusicalTime::TicksPerQuarterNote / 4,			// 240
		EighthNote              = MusicalTime::TicksPerQuarterNote / 2,			// 480
		QuarterNote	            = MusicalTime::TicksPerQuarterNote,				// 960
		HalfNote                = MusicalTime::TicksPerQuarterNote * 2,			// 1920
		WholeNote               = MusicalTime::TicksPerQuarterNote * 4,			// 3840
		Bar                     = std::numeric_limits<int32>::min()+1,          // Ticks depends on Time Signature
		Beat                    = std::numeric_limits<int32>::min()+2           // Ticks depends on Time Signature
	};

	enum EDottedQuantizationTicks : int32
	{
		DottedThirtysecondNote  = (MusicalTime::TicksPerQuarterNote / 16) * 3,	// 180
		DottedSixteenthNote     = (MusicalTime::TicksPerQuarterNote / 8) * 3,	// 360
		DottedEighthNote        = (MusicalTime::TicksPerQuarterNote / 4) * 3,	// 720
		DottedQuarterNote       = (MusicalTime::TicksPerQuarterNote / 2) * 3,	// 1440
		DottedHalfNote          = (MusicalTime::TicksPerQuarterNote) * 3,		// 2880
		DottedWholeNote         = (MusicalTime::TicksPerQuarterNote) * 6,		// 5760
	};

	MUSICENVIRONMENT_API FText GetQuantizationName(int32 Quantization);
	MUSICENVIRONMENT_API EQuartzCommandQuantization QuantiazationIntervalToQuartz(int32 Quantization);

	extern MUSICENVIRONMENT_API TArray<TSharedPtr<int32>> QuantizationOptionsSource;

	enum class EBarBeatFormat
	{
		Position,
		Duration,
		NotAllowed // Don't display bars.beats! Fall back to seconds.
	};
}

struct FGridInterval
{
	double IntervalStartTime;

	int32 IntervalTick;

	FMusicalTime MusicalTime;

	FGridInterval(double InIntervalStartTime, int32 InIntervalTick, FMusicalTime InMusicalTime)
	{
		IntervalStartTime = InIntervalStartTime;
		IntervalTick = InIntervalTick;
		MusicalTime = InMusicalTime;
	}
};

USTRUCT()
struct FFrameBasedTempoPoint
{
	GENERATED_BODY()

	/**
	 * NOTE: This structure treats BPM/Tempo exactly like Midi does to avoid any
	 * floating point precision problems and differences between Midi code and 
	 * this code. It is for that reason that Tempo is stored here a "microseconds
	 * per quarter note" (MPQ) and not a float Bpm. 
	 */

	FFrameBasedTempoPoint()
	{}

	FFrameBasedTempoPoint(float InBpm, int32 InTick, float InAtMillisecond)
		: OnTick(InTick)
		, OnMs(InAtMillisecond)
	{
		SetBpm(InBpm);
	}

	void SetBpm(float InBpm)
	{
		MicrosecondsPerQuarterNote = InBpm >= 0.0f ? static_cast<int32>(60000000.0f / InBpm) : 0;
	}

	bool IsNearlyEqual(const FFrameBasedTempoPoint& Other, float MsTolerance = UE_SMALL_NUMBER) const
	{
		return OnTick == Other.OnTick
			&& MicrosecondsPerQuarterNote == Other.MicrosecondsPerQuarterNote
			&& FMath::IsNearlyEqual(OnMs, Other.OnMs, MsTolerance);
	}

	UPROPERTY()
	int32 MicrosecondsPerQuarterNote = 500000; // 120 bpm default.
	UPROPERTY()
	int32 OnTick = 0;
	UPROPERTY()
	float OnMs = 0.0;

	float Bpm() const
	{
		return MicrosecondsPerQuarterNote == 0 ? 0 : 60000000.0f / static_cast<float>(MicrosecondsPerQuarterNote);
	}

	int32 MsToTick(float Ms) const
	{
		if (Ms == 0.0f)
		{
			return 0;
		}

		return FMath::RoundToInt32(static_cast<double>(OnTick) +
								static_cast<double>(Ms - OnMs) * 1000.0 / 
								static_cast<double>(MicrosecondsPerQuarterNote) *
								static_cast<double>(MusicalTime::TicksPerQuarterNote));
	}

	int32 SecondsToTick(double Second) const
	{
		return MsToTick(static_cast<float>(Second * 1000.0));
	}

	double TickToSeconds(int32 Tick) const
	{
		return static_cast<double>(TickToMs(Tick))/1000.0;
	}

	float TickToMs(int32 Tick) const
	{
		if (Tick == 0)
		{
			return 0.0f;
		}
		float MsPerTick = static_cast<float>(MicrosecondsPerQuarterNote) / (static_cast<float>(MusicalTime::TicksPerQuarterNote) * 1000.0f);
		float DeltaTick = static_cast<float>(Tick - OnTick);
		return OnMs + (DeltaTick * MsPerTick);
	}

	static bool Equals(const TArray<FFrameBasedTempoPoint>& A, const TArray<FFrameBasedTempoPoint>& B)
	{
		if (A.Num() != B.Num())
		{
			return false;
		}
		
		for (int32 Idx = 0; Idx < A.Num(); ++Idx)
		{
			if (!A[Idx].IsNearlyEqual(B[Idx]))
			{
				return false;
			}
		}

		return true;
	}
};

USTRUCT()
struct FFrameBasedTimeSignature
{
	GENERATED_BODY()

	FFrameBasedTimeSignature()
	{}
	FFrameBasedTimeSignature(int32 InNumerator, int32 InDenominator)
		: Numerator(static_cast<int16>(InNumerator))
		, Denominator(static_cast<int16>(InDenominator))
	{}

	UPROPERTY(EditAnywhere, Category="Time Signature")
	int16 Numerator = 4;
	UPROPERTY(EditAnywhere, Category="Time Signature")
	int16 Denominator = 4;

	bool operator==(const FFrameBasedTimeSignature& Other) const
	{
		return Numerator == Other.Numerator && Denominator == Other.Denominator;
	}

	bool operator!=(const FFrameBasedTimeSignature& Other) const
	{
		return Numerator != Other.Numerator || Denominator != Other.Denominator;
	}
};

USTRUCT()
struct FFrameBasedTimeSignaturePoint
{
	GENERATED_BODY()

	FFrameBasedTimeSignaturePoint()
	{}

	FFrameBasedTimeSignaturePoint(const FFrameBasedTimeSignature& InTimeSignature, int32 InOnBar, int32 InOnTick, double InOnFrame)
		: TimeSignature(InTimeSignature)
		, OnBar(InOnBar)
		, OnTick(InOnTick)
		, OnFrame(InOnFrame)
	{
	}

	bool IsNearlyEqual(const FFrameBasedTimeSignaturePoint& Other, double FrameTolerance = UE_DOUBLE_SMALL_NUMBER) const
	{
		return OnTick == Other.OnTick
			&& OnBar == Other.OnBar
			&& TimeSignature == Other.TimeSignature
			&& FMath::IsNearlyEqual(OnFrame, Other.OnFrame, FrameTolerance);
	}

	UPROPERTY()
	FFrameBasedTimeSignature TimeSignature;
	UPROPERTY()
	int32 OnBar = 0;
	UPROPERTY()
	int32 OnTick = 0;
	UPROPERTY()
	double OnFrame = 0.0;

	int32 TicksPerBeat() const
	{
		return MusicalTime::TicksPerQuarterNote * 4 / TimeSignature.Denominator;
	}

	int32 TicksPerBar() const
	{
		return TicksPerBeat() * TimeSignature.Numerator;
	}

	FMusicalTime TickToMusicalTime(int32 InTick) const
	{
		FMusicalTime Result(0, 0, TicksPerBar(), TicksPerBeat());
		int32 TicksPast = InTick - OnTick;
		int32 BarsPassed = TicksPast / TicksPerBar();
		if (TicksPast >= 0)
		{
			Result.Bar = OnBar + BarsPassed;
			Result.TickInBar = TicksPast - (BarsPassed * TicksPerBar());
		}
		else
		{
			Result.Bar = OnBar + BarsPassed;
			int32 TicksRemainder = FMath::Abs(TicksPast) % TicksPerBar();
			if (TicksRemainder > 0)
			{
				Result.Bar--;
				Result.TickInBar = TicksPerBar() - TicksRemainder;
			}
		}
		return Result;
	}

	int32 MusicalTimeToTick(const FMusicalTime& MusicalTime) const
	{
		int32 BarDelta = MusicalTime.Bar - OnBar;
		return OnTick + (BarDelta * MusicalTime.TicksPerBar) + MusicalTime.TickInBar;
	}

	int32 BarToTick(int32 Bar) const
	{
		int32 BarDelta = Bar - OnBar;
		return OnTick + (BarDelta * TicksPerBar());
	}

	int32 BarBeatToTick(int32 Bar, float Beat) const
	{
		int32 BarDelta = Bar - OnBar;
		float TicksIntoBar = Beat * static_cast<float>(TicksPerBeat());
		return OnTick + (BarDelta * TicksPerBar()) + FMath::RoundToInt32(TicksIntoBar);
	}

	static bool Equals(const TArray<FFrameBasedTimeSignaturePoint>& A, const TArray<FFrameBasedTimeSignaturePoint>& B)
	{
		if (A.Num() != B.Num())
		{
			return false;
		}
		
		for (int32 Idx = 0; Idx < A.Num(); ++Idx)
	    {
	        if (A[Idx].IsNearlyEqual(B[Idx]))
	        {
        		return false;
	        }
	    }

		return true;
	}
};

USTRUCT()
struct FTempoTimeSignatureMarker
{
	GENERATED_BODY()

	UPROPERTY()
	TOptional<FFrameBasedTimeSignaturePoint> TimeSigPoint;

	UPROPERTY()
	TOptional<FFrameBasedTempoPoint> TempoPoint;

	UPROPERTY()
	float Seconds = 0.f;
};

UCLASS(MinimalAPI)
class UFrameBasedMusicMap : public UObject
{
	GENERATED_BODY()

public:
	UFrameBasedMusicMap()
		: FrameResolution(0,0)
	{}

	UFrameBasedMusicMap(const FFrameRate& InFrameResolution)
		: FrameResolution(InFrameResolution)
	{
		
	}

	UFrameBasedMusicMap(const FFrameRate& InFrameResolution, float InBpm, const FFrameBasedTimeSignature& TimeSignature)
		: FrameResolution(InFrameResolution)
	{
		TempoMap.Add(FFrameBasedTempoPoint(InBpm, 0, 0.0f));
		BarMap.Add(FFrameBasedTimeSignaturePoint(TimeSignature, 0, 0, 0.0));
	}

	UE_API void SetFrameResolution(const FFrameRate& NewFrameResolution);

	UE_API void Init(float InBpm, const FFrameBasedTimeSignature& InTimeSignature);

	UE_API void SetInitialTempo(float Bpm);
	UE_API float GetInitialTempo() const;
	UE_API void SetInitialTimeSignature(const FFrameBasedTimeSignature& TimeSignature);
	UE_API FFrameBasedTimeSignature GetInitialTimeSignature() const;

	void Clear()
	{
		TempoMap.Empty();
		BarMap.Empty();
	}

	bool IsEmpty() const { return TempoMap.IsEmpty() || BarMap.IsEmpty(); }
	bool IsSimple() const { return TempoMap.Num() == 1 && BarMap.Num() == 1; }

	UE_API double GetFrame(const FMusicalTime& MusicalTime) const;
	UE_API double GetFrame(int32 AtBar) const;
	UE_API FFrameTime GetFrameTime(const FMusicalTime& MusicalTime) const;
	UE_API FFrameTime GetFrameTime(int32 AtBar, float AtBeat) const;
	UE_API FFrameTime GetFrameTime(double FractionalBar) const;
	UE_API FFrameTime GetFrameTime(int32 AtTick) const;

	UE_API double GetSeconds(const FMusicalTime& MusicalTime) const;
	UE_API double GetSeconds(int32 AtBar, float AtBeat = 0.0f) const;

	UE_API FString MakeBarBeatStringForFrame(double DecimalFrame, FrameBasedMusicMap::EBarBeatFormat BarBeatFormat) const;
	UE_API FString MakeBarBeatStringForFrameTime(const FFrameTime& Frame, FrameBasedMusicMap::EBarBeatFormat BarBeatFormat) const;
	UE_API FString MakeBarBeatStringForMusicalTime(const FMusicalTime& MusicalTime, FrameBasedMusicMap::EBarBeatFormat BarBeatFormat) const;
	UE_API FString MakeBarBeatStringForSeconds(double Seconds, FrameBasedMusicMap::EBarBeatFormat BarBeatFormat) const;

	UE_API FMusicalTime GetMusicalTime(int32 Tick) const;
	UE_API FMusicalTime GetMusicalTime(const FFrameTime& FrameTime) const;

	UE_API float GetTempoForFrame(double DecimalFrame) const;
	UE_API float GetTempoForFrameTime(const FFrameTime& FrameTime) const;
	UE_API float GetTempoForSeconds(double Seconds) const;

	UE_API FFrameBasedTimeSignature GetTimeSignatureForFrame(double DecimalFrame) const;
	UE_API FFrameBasedTimeSignature GetTimeSignatureForFrameTime(const FFrameTime& FrameTime) const;
	UE_API FFrameBasedTimeSignature GetTimeSignatureForSeconds(double Seconds) const;
	UE_API int32 GetBeatsInBar(int32 Bar) const;
	UE_API int32 GetTicksInBar(int32 Bar) const;
	
	UE_API FFrameTime RemapFrameTime(FFrameTime SourceTime, const UFrameBasedMusicMap* OldMap) const;

	UE_API FMusicalTime BarsToMusicalTime(double BarsFromStart) const;

	enum class EQuantizeDirection
	{
		Nearest = 0,
		Ceiling = 1,
		Floor = 2
	};
	UE_API FMusicalTime Quantize(const FMusicalTime& MusicalTime, int32 QuantizationInterval, EQuantizeDirection Direction = EQuantizeDirection::Nearest) const;
	UE_API int32 QuantizeToTick(const FMusicalTime& MusicalTime, int32 QuantizationInterval, EQuantizeDirection Direction = EQuantizeDirection::Nearest) const;

	/**
	* FMusicalTime Add(const FMusicalTime& MusicalTime, int32 QuantizationInterval) const;
	* 
	* Increment the supplied MusicalTime by the specified QuantizationInterval. 
	* NOTE: This does NOT quantize anything. QuantizationInterval is just used 
	* as a convenient way to specify a musical subdivision. You can use any of the 
	* TripletQuantizationTicks, WholeQuantizationTicks, or DottedQuantizationTicks
	* enum values. AND... you can negate them to go back in time.
	* 
	* @param MusicalTime			Time to be added to
	
	* @param QuantizationInterval	One of the values from...
	*                               TripletQuantizationTicks
	*								WholeQuantizationTicks
	*								DottedQuantizationTicks
	* 
	*                                 
	*/
	UE_API FMusicalTime Add(const FMusicalTime& MusicalTime, int32 QuantizationInterval) const;

	UPROPERTY()
	TArray<FFrameBasedTempoPoint> TempoMap;
	UPROPERTY()
	TArray<FFrameBasedTimeSignaturePoint> BarMap;
	UPROPERTY()
	FFrameRate FrameResolution;

	/**
	 * Appends a time signature change to the end of the Bar Map.
	 * Ensures that this time signature change's tick is after the last one in the map
	 * 
	 * @param AtTick
	 * @param AtBar
	 * @param Numerator
	 * @param Denominator
	 */
	UE_API void AddTimeSignature(int32 AtTick, int32 AtBar, int16 Numerator, int16 Denominator);

	/**
	 * Appends a tempo change to the end of the
	 * Ensures that this tempo change's tick is after the last one in the map
	 * 
	 * @param AtTick
	 * @param AtMs
	 * @param Bpm
	 */
	UE_API void AddTempo(int32 AtTick, float AtMs, float Bpm);

	/**
	 * Inserts a Time Signature change at the specified Bar, avoiding duplicates as needed.
	 * Time Signature changes after this point will recalculate their ticks and frame times
	 * to happen on the new bar lines based on the inserted time signature change
	 *
	 * This will not insert a time signature change if the same change already exists immediately before this one being inserted
	 * If there is the same time signature change immediately after this one being inserted, it will be removed.
	 *
	 * @param AtBar
	 * @param Numerator
	 * @param Denominator
	 */
	UE_API void InsertTimeSignature(int32 AtBar, int16 Numerator, int16 Denominator);

	/**
	 * Inserts a Tempo Change at the specified tick, avoiding duplicates as needed.
	 * Time signature and tempo changes after this point will recalculate their frame times and ms respectively
	 * based on the new tempo change
	 * 
	 * This will not insert a tempo change if the same change already exits immediately before
	 * If the same tempo change exists immediately after this one being inserted, it will be removed
	 *
	 * @param AtTick
	 * @param Bpm
	 */
	UE_API void InsertTempo(int32 AtTick, float Bpm);

	/**
	 * Inserts a Time Signature change at the specified Bar of the Musical Time (ignoring beats), avoiding duplicates as needed.
	 * Time Signature changes after this point will recalculate their ticks and frame times
	 * to happen on the new bar lines based on the inserted time signature change
	 *
	 * This will not insert a time signature change if the same change already exists immediately before this one being inserted
	 * If there is the same time signature change immediately after this one being inserted, it will be removed.
	 *
	 * @param AtMusicalTime
	 * @param TimeSignature
	 */
	UE_API void InsertTimeSignature(FMusicalTime AtMusicalTime, const FFrameBasedTimeSignature& TimeSignature);

	/**
	 * Inserts a Tempo Change at the specified MusicalTime, avoiding duplicates as needed.
	 * Time signature and tempo changes after this point will recalculate their frame times and ms respectively
	 * based on the new tempo change
	 *
	 * This will not insert a tempo change if the same change already exits immediately before
	 * If the same tempo change exists immediately after this one being inserted, it will be removed
	 *
	 * @param AtMusicalTime
	 * @param Bpm
	 */
	UE_API void InsertTempo(FMusicalTime AtMusicalTime, float Bpm);

	/**
	 * Removes the time signature change at the specified bar, if it exists
	 * Time Signature changes after this point will recalculate their ticks and frame times
	 * to happen on the new bar lines based on the inserted time signature change
	 * 
	 * @param AtBar
	 * @return true if a time signature change was removed
	 */ 
	UE_API bool RemoveTimeSignature(int32 AtBar);

	/**
	 * Removes the tempo change at the specified tick, if it exists
	 * Time signature and tempo changes after this point will recalculate their frame times and ms respectively
	 * based on the new tempo change
	 * 
	 * @param AtTick
	 * @return true if the tempo change was removed
	 */
	UE_API bool RemoveTempo(int32 AtTick);

	/**
	 * Removes the time signature change at the specified Musical Time, if it exists
	 *
	 * Note: Time Signature changes only happen on bar bounderies.
	 * A Musical Time that is not on whole bare is guaranteed to fail
	 *
	 * @param AtMusicalTime
	 * @return true if a time signature change was removed
	 */ 	
	UE_API bool RemoveTimeSignature(FMusicalTime AtMusicalTime);
	
	/**
	 * Removes the tempo change at the specified Musical Time, if it exists
	 *
	 * @param AtMusicalTime
	 * @return true if the tempo change was removed
	 */
	UE_API bool RemoveTempo(FMusicalTime AtMusicalTime);

	/**
	 * @param AtBar
	 * @return true if there is a time signature change at the given bar
	 */
	UE_API bool HasTimeSignatureChange(int32 AtBar);

	/**
	 * @param AtTick
	 * @return true if there is a tempo change at the given tick
	 */
	UE_API bool HasTempoChange(int32 AtTick);

	/**
	 * @param AtMusicalTime
	 * @return true if there is a time signature change at the exact musical time
	 */
	UE_API bool HasTimeSignatureChange(FMusicalTime AtMusicalTime);

	/**
	 * @param AtMusicalTime
	 * @return true if there is a tempo change at the exact musical time
	 */
	UE_API bool HasTempoChange(FMusicalTime AtMusicalTime);
	
	
	UE_API void GatherSubdivisonsForTimespan(int32 StartTick, int32 EndTick, int32 QuantizationInterval, TArray<FGridInterval>& OutSubdivisions) const;

	UE_API void GatherBarsForTimespan(int32 StartBar, int32 EndBar, int32 QuantizationInterval, TArray<FGridInterval>& OutBars) const;

	UE_API void GetGridMetricsMusic(float InViewStartSeconds, float InViewEndSeconds, float PixelsPerSecond, float MinTickPx, float DesiredMajorTickPx, TArray<FGridInterval>& OutMajorIntervals, TArray<FGridInterval>& OutMinorIntervals) const;

	UE_API void ComputeGridSpacingAtTick(int32 TickToCompute, float DesiredMajorTickSeconds, int32& OutMajorIntervalTicks) const;

	UE_API void ComputeMinorGridSpacing(int32 StartTick, int32 EndTick, TArray<FGridInterval>& OutMinorIntervals) const;

	UE_API TArray<FFrameBasedTempoPoint> GetTempoMetrics(float InViewStartSeconds, float InViewEndSeconds) const;

	UE_API void GetTempoTimeSignatureMarkers(float InStartSeconds, float InEndSeconds, TArray<FTempoTimeSignatureMarker>& OutMarkers) const;

private:
	UE_API const FFrameBasedTimeSignaturePoint* GetTimeSignaturePointForTick(int32 Tick) const;
	UE_API const FFrameBasedTimeSignaturePoint* GetTimeSignaturePointForBar(int32 Bar) const;
	UE_API const FFrameBasedTimeSignaturePoint* GetTimeSignaturePointForFrame(double DecimalFrame) const;
	UE_API const FFrameBasedTimeSignaturePoint* GetTimeSignaturePointForFrameTime(const FFrameTime& Frame) const;
	UE_API const FFrameBasedTimeSignaturePoint* GetTimeSignaturePointForSeconds(double Seconds) const;

	UE_API const FFrameBasedTempoPoint* GetTempoPointForTick(int32 Tick) const;
	UE_API const FFrameBasedTempoPoint* GetTempoPointForFrame(double DecimalFrame) const;
	UE_API const FFrameBasedTempoPoint* GetTempoPointForFrameTime(const FFrameTime& Frame) const;
	UE_API const FFrameBasedTempoPoint* GetTempoPointForSeconds(double Seconds) const;

	UE_API FString MakeBarBeatString(double DecimalFrame, const FFrameBasedTimeSignaturePoint* TimeSignature, const FFrameBasedTempoPoint* Tempo, FrameBasedMusicMap::EBarBeatFormat BarBeatFormat) const;
};

#undef UE_API
