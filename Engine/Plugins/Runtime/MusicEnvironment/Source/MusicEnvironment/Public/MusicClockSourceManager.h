// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayTagContainer.h"
#include "MusicEnvironmentClockSource.h"
#include "UObject/WeakInterfacePtr.h"

#include "MusicClockSourceManager.generated.h"

#define UE_API MUSICENVIRONMENT_API

UCLASS(MinimalAPI, BlueprintType)
class UMusicClockSourceManager : public UObject
{
	GENERATED_BODY()
public:
	
	UFUNCTION(BlueprintCallable, Category="Music|Clock")
	UE_API TScriptInterface<IMusicEnvironmentClockSource> FindClock(const FGameplayTag& Tag, bool bExactMatch = true);

	UFUNCTION(BlueprintCallable, Category="Music|Clock")
	UE_API void AddTaggedClock(const FGameplayTag& Tag, TScriptInterface<IMusicEnvironmentClockSource> InClock);
	UE_API void AddTaggedClock(const FGameplayTag& Tag, UObject* InClock);

	UFUNCTION(BlueprintCallable, Category="Music|Clock")
	UE_API void RemoveTaggedClock(TScriptInterface<IMusicEnvironmentClockSource> InClock);
	UE_API void RemoveTaggedClock(UObject* InClock);

	UFUNCTION(BlueprintCallable, Category="Music|Clock")
	UE_API void RemoveClockWithTag(const FGameplayTag& Tag);

	UFUNCTION(BlueprintCallable, Category="Music|Clock")
	UE_API TScriptInterface<IMusicEnvironmentClockSource> GetGlobalMusicClockAuthority();

	UFUNCTION(BlueprintCallable, Category = "Music|Clock")
	UE_API bool PushGlobalMusicClockAuthority(TScriptInterface<IMusicEnvironmentClockSource> InClock);
	UE_API bool PushGlobalMusicClockAuthority(UObject* InClock);

	UFUNCTION(BlueprintCallable, Category = "Music|Clock")
	UE_API void RemoveGlobalClockAuthority(TScriptInterface<IMusicEnvironmentClockSource> InClock);
	UE_API void RemoveGlobalClockAuthority(UObject* InClock);

	UFUNCTION(BlueprintCallable, Category = "Music|Clock")
	UE_API void PopMusicClockAuthority();

private:
	TMap<FGameplayTag, TWeakInterfacePtr<IMusicEnvironmentClockSource>> TaggedClocks;
	TArray<TWeakInterfacePtr<IMusicEnvironmentClockSource>> GlobalClockSourceStack;

	void CleanMap();
	void CleanStack();
};

#undef UE_API
