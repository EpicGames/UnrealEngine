// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "FrameBasedMusicMap.h"
#include "Misc/MusicalTime.h"
#include "UObject/Interface.h"

#include "MusicEnvironmentClockSource.generated.h"

#define UE_API MUSICENVIRONMENT_API

// This class does not need to be modified.
UINTERFACE(MinimalAPI, BlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class UMusicEnvironmentClockSource : public UInterface
{
	GENERATED_BODY()
};

/**
 * 
 */
class IMusicEnvironmentClockSource
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = "Music|Clock")
	UE_API virtual void GetCurrentBarBeat(float& Bar, float& BeatInBar);

	virtual float GetPositionSeconds() const = 0;
	virtual FMusicalTime GetPositionMusicalTime() const = 0;
	virtual FMusicalTime GetPositionMusicalTime(const FMusicalTime& SourceSpaceOffset) const = 0;
	virtual int32 GetPositionAbsoluteTick() const = 0;
	virtual int32 GetPositionAbsoluteTick(const FMusicalTime& SourceSpaceOffset) const = 0;
	virtual FMusicalTime Quantize(const FMusicalTime& MusicalTime, int32 QuantizationInterval, UFrameBasedMusicMap::EQuantizeDirection Direction = UFrameBasedMusicMap::EQuantizeDirection::Nearest) const = 0;
	virtual bool CanAuditionInEditor() const = 0;
};

#undef UE_API // MUSICENVIRONMENT_API
