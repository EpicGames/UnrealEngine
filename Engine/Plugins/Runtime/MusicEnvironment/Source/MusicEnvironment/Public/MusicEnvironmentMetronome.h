// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "FrameBasedMusicMap.h"
#include "MusicEnvironmentClockSource.h"
#include "UObject/Interface.h"

#include "MusicEnvironmentMetronome.generated.h"

#define UE_API MUSICENVIRONMENT_API

// This class does not need to be modified.
UINTERFACE(MinimalAPI)
class UMusicEnvironmentMetronome : public UMusicEnvironmentClockSource
{
	GENERATED_BODY()
};

/**
 * A music environment system that can spawn metronomes will have to return an 
 * instance of a UObject that implements this interface from its "MovieSceneMetronomeSpawner". 
 */
class IMusicEnvironmentMetronome : public IMusicEnvironmentClockSource
{
	GENERATED_BODY()

	// Add interface functions to this class. This is the class that will be inherited to implement this interface.
public:
	UE_API void SetMusicMap(UFrameBasedMusicMap* InMusicMap);

	virtual bool Initialize(UWorld* InWorld) = 0;
	virtual void Tick(float DeltaSeconds) = 0;
	virtual void Start(double FromSeconds = 0.0) = 0;
	virtual void Seek(double ToSeconds) = 0;
	virtual void Stop() = 0;
	virtual void Pause() = 0;
	virtual void Resume() = 0;

	UE_API bool SetTempo(float Bpm);
	virtual float GetCurrentTempo() const = 0;
	virtual double GetCurrentPositionSeconds() const = 0;
	
	UE_API void SetSpeed(float Speed);
    virtual float GetCurrentSpeed() const = 0;
    	
	UE_API void SetVolume(float InVolumeLinear);
	virtual float GetCurrentVolume() const = 0;

	UE_API void SetMuted(bool bInMuted);
	virtual bool IsMuted() const = 0;
	
protected:
	virtual void OnMusicMapSet() = 0;
	virtual void OnSetSpeed(const float Speed) = 0;
	virtual bool OnSetTempo(const float Bpm) = 0;
	virtual void OnSetVolume(float NewVolumeLinear) = 0;
	virtual void OnSetMuted(bool bNewMuted) = 0;
	
	TStrongObjectPtr<UFrameBasedMusicMap> MusicMap;
};

#undef UE_API
