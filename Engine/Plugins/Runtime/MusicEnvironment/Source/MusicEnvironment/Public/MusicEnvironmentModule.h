// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

/**
 * The public interface of the MusicEnvironment module
 */
class FMusicEnvironmentModule
	: public IModuleInterface
{
public:

	static inline FMusicEnvironmentModule& Get()
	{
		return FModuleManager::LoadModuleChecked<FMusicEnvironmentModule>("MusicEnvironment");
	}
};
