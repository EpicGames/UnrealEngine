// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Subsystems/EngineSubsystem.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakInterfacePtr.h"

#include "MusicEnvironmentSubsystem.generated.h"

#define UE_API MUSICENVIRONMENT_API

// Forward Declarations
class IMusicEnvironmentMetronome;
class UMusicClockSourceManager;

UCLASS(MinimalAPI, BlueprintType, Category = "Music", DisplayName = "MusicEnvironmentSubsystem")
class UMusicEnvironmentSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()
public:
	static UE_API UMusicEnvironmentSubsystem& Get();
	
	UFUNCTION(BlueprintCallable, Category = "Music|Clock")
	UE_API UMusicClockSourceManager* GetClockSourceManager();

	UE_API bool SetMetronomeClass(const UClass* InMetronomeType);

	UFUNCTION(BlueprintCallable, Category = "Music|Clock")
	UE_API bool CanSpawnMetronome() const;

	UFUNCTION(BlueprintCallable, Category = "Music|Clock")
	UE_API virtual TScriptInterface<IMusicEnvironmentMetronome> SpawnMetronome(UObject* Outer, FName Name = NAME_None);

private:
	UPROPERTY(Transient)
	TObjectPtr<UMusicClockSourceManager> ClockSourceManager;

	UPROPERTY(Transient)
	TSoftClassPtr<UObject> MetronomeType;
};

MUSICENVIRONMENT_API DECLARE_LOG_CATEGORY_EXTERN(LogMusicEnvironment, Log, All);

#undef UE_API
