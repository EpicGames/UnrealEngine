// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MusicalAsset.h"
#include "MusicEnvironmentClockSource.h"
#include "UObject/Interface.h"

#include "MusicHandle.generated.h"

#define UE_API MUSICENVIRONMENT_API

/**
 * This file defines:
 * ==================
 * UMusicHandle: The UInterface derived class needed to support Unreal's interface system. This 
 *               class definition stays empty.
 * 
 * IMusicHandle: The actual pure virtual definition of the interface. This is what 
 *               derived classes will inherit from. AND... IMusicalAssets return 
 *               TScriptInterface<IMusicHandle> when they are asked to instantiate themselves.
 *               BUT NOTE... If a non-UCLASS or a non-USTRUCT asks an IMusicalAsset to instantiate 
 *               itself there may not be anything holding a strong reference to the object the
 *               script interface is pointing to. More on this below.
 * 
 * FMusicHandle: This is a convenience type definition that amounts to a TScriptInterface<IMusicHandle>. 
 *               ULASSS and USTRUCT things can hold one of these as a member and mark it as a UPROPERTY()
 *               so the object that implements the script interface will not be garbage collected.
 *               It looks cleaner.
 * 
 * FStrongMusicHandle: This wraps a TScriptInterface<IMusicHandle> to hold a TStrongObjectPtr to 
 *               the UObject that implements the interface. It is like having a TStrongInterfacePtr<>,
 *               a type that doesn't exist in the engine. This allows non-UCLASS, non-USTRUCT things to
 *               hold music handles, and for the UStuff inside of those handles to not be garbage collected
 *               until the FStrongMusicHandle is destroyed.
 */

class UAudioComponent;

UENUM(BlueprintType)
enum class EMusicHanldeClockValidity : uint8
{
	ClockValid,
	ClockInvalid
};

UENUM(BlueprintType)
enum class EMusicHanldeTransportState : uint8
{
	PreparingToPlay,
	ReadyToPlay,
	Playing,
	Paused,
	Stale,
};

 /**
 * UMusicHandle: The UInterface derived class needed to support Unreal's interface system. This 
 *               class definition stays empty.
 */
UINTERFACE(MinimalAPI, BlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class UMusicHandle : public UInterface
{
	GENERATED_BODY()
};

/**
 * IMusicHandle: The actual pure virtual definition of the UMusicHandle interface. This is what 
 *               derived classes will inherit from. AND... IMusicalAssets return 
 *               TScriptInterface<IMusicHandle> when they are asked to instantiate themselves.
 *               BUT NOTE... If a non-UCLASS or a non-USTRUCT asks an IMusicalAsset to instantiate 
 *               itself there may not be anything holding a strong reference to the object the
 *               script interface is pointing to. More on this below.
 */
class IMusicHandle
{
public:
	GENERATED_BODY()

	UFUNCTION(BlueprintCallable, Category = "Music")
	virtual bool IsValid() const = 0;

	virtual void Tick(float DeltaSeconds) = 0;

	//** Transport
	UFUNCTION(BlueprintCallable, Category = "Music")
	virtual bool IsReadyToPlay(float FromSeconds = 0.0f) { return true; }
	UFUNCTION(BlueprintCallable, Category = "Music")
	virtual bool Play(float FromSeconds = 0.0f) = 0;
	UFUNCTION(BlueprintCallable, Category = "Music")
	virtual void Pause() = 0;
	UFUNCTION(BlueprintCallable, Category = "Music")
	virtual void Continue() = 0;
	UFUNCTION(BlueprintCallable, Category = "Music")
	UE_API virtual void Stop();
	UFUNCTION(BlueprintCallable, Category = "Music")
	UE_API virtual void Kill();

	operator bool()
	{
		return IsValid();
	}

	UFUNCTION(BlueprintCallable, Category = "Music", Meta = (ExpandEnumAsExecs = "Branches"))
	UE_API virtual void BranchOnTransportState(EMusicHanldeTransportState& Branches) const;

	UFUNCTION(BlueprintCallable, Category = "Music")
	virtual EMusicHanldeTransportState GetTransportState() const { return CurrentTransportState; }

	UFUNCTION(BlueprintCallable, Category = "Music")
	virtual bool IsUsingAsset(const TScriptInterface<IMusicalAsset> Asset) const { return IsUsingAsset(Asset.GetObject()); }
	virtual bool IsUsingAsset(const UObject* Asset) const = 0;

	UFUNCTION(BlueprintCallable, Category = "Music|Clock")
	virtual TScriptInterface<IMusicEnvironmentClockSource> GetMusicClockSource() = 0;

	UFUNCTION(BlueprintCallable, Category = "Music|Clock", Meta = (ExpandEnumAsExecs = "Branches"))
	UE_API virtual void GetCurrentBarBeat(float& Bar, float& BeatInBar, EMusicHanldeClockValidity& Branches);

	UFUNCTION(BlueprintCallable, Category = "Music|Clock")
	UE_API virtual void BecomeGlobalMusicClockAuthority();

	UFUNCTION(BlueprintCallable, Category = "Music|Clock")
	UE_API virtual void RelinquishGlobalMusicClockAuthority();
	
	UFUNCTION(BlueprintCallable, Category = "Music|Clock")
	UE_API virtual void RegisterAsTaggedClock(const FGameplayTag& Tag);

	UFUNCTION(BlueprintCallable, Category = "Music|Clock")
	UE_API virtual void UnregisterAsTaggedClock(const FGameplayTag& Tag = FGameplayTag());

protected:
	virtual void Stop_Internal() = 0;
	virtual void Kill_Internal() = 0;

	EMusicHanldeTransportState CurrentTransportState = EMusicHanldeTransportState::PreparingToPlay;

private:
	bool IsOnGlobalClockAuthorityStack = false;
	FGameplayTagContainer RegisteredToTags;
};

using FMusicHandle = TScriptInterface<IMusicHandle>;

/** 
 * FStrongMusicHandle: This wraps a TScriptInterface<IMusicHandle> to hold a TStrongObjectPtr to 
 *               the UObject that implements the interface. It is like having a TStrongInterfacePtr<>,
 *               a type that doesn't exist in the engine. This allows non-UCLASS, non-USTRUCT things to
 *               hold music handles, and for the UStuff inside of those handles to not be garbage collected
 *               until the FStrongMusicHandle is destroyed.
 */
struct FStrongMusicHandle
{
public:
	inline FStrongMusicHandle() = default;
	inline FStrongMusicHandle(const FStrongMusicHandle& Other) = default;
	inline FStrongMusicHandle(FStrongMusicHandle&& Other) = default;
	inline ~FStrongMusicHandle() = default;
	inline FStrongMusicHandle& operator=(const FStrongMusicHandle& Other) = default;
	inline FStrongMusicHandle& operator=(FStrongMusicHandle&& Other) = default;

	/**
	 * A constructor that can take a pointer to any UObject that implements the IMusicHandle interface
	 */ 
	template<
		typename U
		UE_REQUIRES(std::is_convertible_v<U, TCopyQualifiersFromTo_T<U, UObject>*>)
	>
	FStrongMusicHandle(U&& MusicHandleObject)
	{
		InterfaceInstance = Cast<IMusicHandle>(ImplicitConv<TCopyQualifiersFromTo_T<U, UObject>*>(MusicHandleObject));
		if (InterfaceInstance != nullptr)
		{
			ObjectInstance = TStrongObjectPtr<UObject>(MusicHandleObject);
		}
	}

	inline FStrongMusicHandle& operator=(FMusicHandle Other)
	{
		ObjectInstance = TStrongObjectPtr<UObject>(Other.GetObject());
		InterfaceInstance = Other.GetInterface();
		return *this;
	}

	template<
		typename U
		UE_REQUIRES(std::is_convertible_v<U, TCopyQualifiersFromTo_T<U, UObject>*>)
	>
	inline FStrongMusicHandle& operator=(U& Other)
	{
		// Regardless of whether we successfully get an interface from the object,
		// we need to release any strong pointer we may already have to another object...
		ObjectInstance = nullptr;
		InterfaceInstance = Cast<IMusicHandle>(ImplicitConv<TCopyQualifiersFromTo_T<U, UObject>*>(&Other));
		if (InterfaceInstance != nullptr)
		{
			ObjectInstance = TStrongObjectPtr<UObject>(&Other);
		}
		return *this;
	}

	inline FStrongMusicHandle& operator=(std::nullptr_t)
	{
		ObjectInstance = nullptr;
		InterfaceInstance = nullptr;
		return *this;
	}

	inline IMusicHandle& operator*() const
	{
		check(InterfaceInstance);
		return *InterfaceInstance;
	}

	inline IMusicHandle* operator->() const
	{
		check(InterfaceInstance);
		return InterfaceInstance;
	}

	operator bool() const
	{
		return InterfaceInstance != nullptr;
	}

private:
	TStrongObjectPtr<UObject> ObjectInstance;
	IMusicHandle* InterfaceInstance = nullptr;
};

UCLASS()
class UMusicHandleBlueprintHelpers : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	UFUNCTION(BlueprintCallable, Category = "Music", Meta = (ExpandEnumAsExecs = "Branches"))
	static void BranchOnTransportState(TScriptInterface<IMusicHandle> Handle, EMusicHanldeTransportState& Branches);

	UFUNCTION(BlueprintCallable, Category = "Music")
	static EMusicHanldeTransportState GetTransportState(TScriptInterface<IMusicHandle> Handle);
};

#undef UE_API // MUSICENVIRONMENT_API
