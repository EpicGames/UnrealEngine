// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/FrameNumber.h"
#include "UObject/Interface.h"

#include "MusicMapSource.generated.h"

class UFrameBasedMusicMap;

#define UE_API MUSICENVIRONMENT_API

struct FMarkerProviderEntry
{
	FString Label;
	FFrameNumber FrameNumber;

	FMarkerProviderEntry(const FString& InLabel, FFrameNumber InFrameNumber)
		: Label(InLabel)
		, FrameNumber(InFrameNumber)
	{}
};

USTRUCT()
struct FMarkerProviderResults
{
	GENERATED_BODY()

	struct FChannel
	{
		FString Name;
		TArray<FMarkerProviderEntry> Markers;

		FChannel(const FString& InName)
			: Name(InName)
		{}
	};

	bool IsEmpty() const { return Channels.IsEmpty(); }

	TArray<FChannel> Channels;
};

UINTERFACE(MinimalAPI)
class UMusicMapSource : public UInterface
{
	GENERATED_BODY()
};

class IMusicMapSource
{
public:

	GENERATED_BODY()

	UFUNCTION()
	virtual void CreateFrameBasedMusicMap(UFrameBasedMusicMap* Map) const { return; }

	UFUNCTION()
	virtual FMarkerProviderResults GatherMarkers(const UFrameBasedMusicMap* Map) const { return FMarkerProviderResults(); }

	UFUNCTION()
	virtual float GetSongLengthSeconds() const { return 0.0f; }
};

#undef UE_API // MUSICENVIRONMENT_API