// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "GameplayTagContainer.h"
#include "MusicTypes/MusicMapSource.h"
#include "MusicalAsset.generated.h"

#define UE_API MUSICENVIRONMENT_API

/**
 * IMusicalAssets are assets that can...
 *   - Provide a UFrameBasedMusicMap that describes the musical form (tempo and time signature) of the
 *     music in the resolution provided by the requestor.
 *   - Instantiate themselves when they are told to Play or PrepareToPlay. (NOTE: THIS FUNCTIONALITY
 *     BELONGS IN AN IAudioEmitter class/interface WHEN ONE EXISTS! It is here given the current
 *     lack of such an emitter system in Unreal Engine. Emitters should be asked to play an asset
 *     and the caller should get a handle back.)
 *
 * This file defines:
 * ==================
 * UMusicalAsset: The UInterface derived class needed to support Unreal's interface system. This
 *                class definition stays empty.
 *
 * IMusicalAsset: The actual pure virtual definition of the interface. This is what
 *                derived classes will inherit from. 
 * 
 * FMusicalAsset: This is a convenience type definition that amounts to a TScriptInterface<IMusicalAsset>. 
 *                ULASSS and USTRUCT things can hold one of these as a member and mark it as a UPROPERTY()
 *                so the object that implements the script interface will not be garbage collected.
 *                It looks cleaner.
 */

class UMusicHandle;
class UAudioComponent;
class IMusicHandle;

/**
 * UMusicalAsset: The UInterface derived class needed to support Unreal's interface system. This
 *                class definition stays empty.
 */
UINTERFACE(MinimalAPI, BlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class UMusicalAsset : public UMusicMapSource
{
	GENERATED_BODY()
};

/**
 * IMusicalAsset: The actual pure virtual definition of the interface. This is what
 *                derived classes will inherit from. It is an asset that has a
 *                FrameBasedMusicMap and can be played as music (returning a 
 *                'music handle' that becomes owned by the caller.)
 */
class IMusicalAsset : public IMusicMapSource
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Music", Meta = (DefaultToSelf = "PlaybackContext"))
	UE_API virtual TScriptInterface<IMusicHandle> PrepareToPlay(UObject* PlaybackContext, 
		UAudioComponent* OnComponent = nullptr,
		float FromSeconds = 0.0f,
		bool BecomeAuthoritativeClock = false,
		const FGameplayTag RegisterAsTaggedClock = FGameplayTag(),
		bool IsAudition = false);

	UFUNCTION(BlueprintCallable, Category = "Music", Meta = (DefaultToSelf = "PlaybackContext"))
	UE_API virtual TScriptInterface<IMusicHandle> Play(UObject* PlaybackContext,
		UAudioComponent* OnComponent = nullptr,
		float FromSeconds = 0.0f,
		bool BecomeAuthoritativeClock = false,
		const FGameplayTag RegisterAsTaggedClock = FGameplayTag(),
		bool IsAudition = false);

protected:
	virtual TScriptInterface<IMusicHandle> PrepareToPlay_Internal(UObject* PlaybackContext,
		UAudioComponent* OnComponent = nullptr,
		float FromSeconds = 0.0f,
		bool IsAudition = false) = 0;

	virtual TScriptInterface<IMusicHandle> Play_Internal(UObject* PlaybackContext,
		UAudioComponent* OnComponent = nullptr,
		float FromSeconds = 0.0f,
		bool IsAudition = false) = 0;
};

/** 
 * FMusicalAsset: This is a convenience type definition.
 */
using FMusicalAsset = TScriptInterface<IMusicalAsset>;

#undef UE_API // MUSICENVIRONMENT_API
