// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MusicalEnums.generated.h"

UENUM(BlueprintType)
enum class EMusicalSyncPointCaptureMode : uint8
{
	DirectMapped              UMETA(ToolTip="Bar 1 | Beat 1 of the level sequence is mapped directly to Bar 1 | Beat 1 of the driving music clock. If needed the sequence is 'seeked' to match the current time of the music clock."),
	Immediate                 UMETA(ToolTip="Bar 1 | Beat 1 of the level sequence will be mapped to the current position of the music clock. Now quantization is done. While the sequence will advance in a locked-step, tempo matched way, its offset will be maintained. CAREFUL! You probably want to use some sort of qunatized capture."),
	NearestQuantizedPosition  UMETA(ToolTip="Bar 1 | Beat 1 of the level sequence is mapped to the NEAREST specified quantized musical subdivision. May be before or after the current musical time. Playback will start right away if the nearest subdivision is in the past, otherwise playback will start when the next subdivision arrives."),
	NextQuantizedPosition     UMETA(ToolTip="Bar 1 | Beat 1 of the level sequence is mapped to the NEXT specified quantized musical subdivision. Will be on or after the current musical time. Playback will start when that subdivision arrives."),
	PreviousQuantizedPosition UMETA(ToolTip="Bar 1 | Beat 1 of the level sequence is mapped to the Previous specified quantized musical subdivision. Will be on or before the current musical time. Playback will start right away."),
};

UENUM(BlueprintType)
enum class EMusicalTimeSignatureRemapMode : uint8
{
	None                    UMETA(ToolTip="No attempt is made to maintain 'bar synchronization' if there are mismatched time signature."),
	ScaleBars               UMETA(ToolTip="Scale each bar as necessary to match bar lengths in music clock."),
	CropOrLoopBars          UMETA(ToolTip="Crop or loop bars as necessary to align downbeats between the level sequence and the music clock."),
	ScaleBeatsAndCropOrLoop UMETA(ToolTip="Scale beats to match music clock beat lengths, and THEN crop or loop bars as necessary to align downbeats between the level sequence and the music clock.")
};
