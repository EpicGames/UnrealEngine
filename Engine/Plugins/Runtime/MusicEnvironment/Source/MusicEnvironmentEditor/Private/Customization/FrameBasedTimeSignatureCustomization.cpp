// Copyright Epic Games, Inc. All Rights Reserved.

#include "FrameBasedTimeSignatureCustomization.h"

#include "DetailWidgetRow.h"
#include "PropertyHandle.h"
#include "Widgets/Input/SFrameBasedTimeSignatureInput.h"

void FFrameBasedTimeSignatureCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	MyPropertyHandle = PropertyHandle.ToSharedPtr();
	HeaderRow
	.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SFrameBasedTimeSignatureInput)
		.Value_Lambda([this]() -> FFrameBasedTimeSignature
		{
			if (!MyPropertyHandle.IsValid())
			{
				return FFrameBasedTimeSignature();
			}
			int16 Numerator;
			int16 Denominator;
			TSharedPtr<IPropertyHandle> NumHandle = MyPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FFrameBasedTimeSignature, Numerator));
			TSharedPtr<IPropertyHandle> DenomHandle = MyPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FFrameBasedTimeSignature, Denominator));
			check(NumHandle);
			check(DenomHandle);
			if (NumHandle->GetValue(Numerator) == FPropertyAccess::Fail)
			{
				return FFrameBasedTimeSignature();
			}
			
			if (DenomHandle->GetValue(Denominator) == FPropertyAccess::Fail)
			{
				return FFrameBasedTimeSignature();
			}
			
			return FFrameBasedTimeSignature(Numerator, Denominator);
		})
		.OnValueCommitted_Lambda([this](const FFrameBasedTimeSignature& NewValue, ETextCommit::Type)
		{
			if (!MyPropertyHandle.IsValid())
			{
				return;
			}
			
			TSharedPtr<IPropertyHandle> NumHandle = MyPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FFrameBasedTimeSignature, Numerator));
            TSharedPtr<IPropertyHandle> DenomHandle = MyPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FFrameBasedTimeSignature, Denominator));
            check(NumHandle);
            check(DenomHandle);
			NumHandle->SetValue(NewValue.Numerator);
			DenomHandle->SetValue(NewValue.Denominator);
		})
	];
}

void FFrameBasedTimeSignatureCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	
}
