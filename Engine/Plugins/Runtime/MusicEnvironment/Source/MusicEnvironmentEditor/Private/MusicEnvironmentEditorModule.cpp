// Copyright Epic Games, Inc. All Rights Reserved.

#include "MusicEnvironmentEditorModule.h"
#include "Modules/ModuleInterface.h"
#include "PropertyEditorModule.h"
#include "Customization/FrameBasedTimeSignatureCustomization.h"
#include "Modules/ModuleManager.h"

class FMusicEnvironmentEditorModule : public IModuleInterface
{
public:
 
	virtual void StartupModule() override
	{
		// register detail customizations
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyEditorModule.RegisterCustomPropertyTypeLayout("FrameBasedTimeSignature", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FFrameBasedTimeSignatureCustomization::MakeInstance));
	}
 
	virtual void ShutdownModule() override
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyEditorModule.UnregisterCustomPropertyTypeLayout("FrameBasedTimeSignature");
	}
};

IMPLEMENT_MODULE(FMusicEnvironmentEditorModule, MusicEnvironmentEditor);