// Copyright Epic Games, Inc. All Rights Reserved.


#include "Widgets/Input/SFrameBasedTimeSignatureInput.h"

#include "FrameBasedMusicMap.h"
#include "Internationalization/Regex.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "SFrameBasedTimeSignatureInput"

void SFrameBasedTimeSignatureInput::Construct(const FArguments& InArgs)
{
	ValueAttribute = InArgs._Value;
	MaxNumerator = InArgs._MaxNumerator;
	MaxDenominator = InArgs._MaxDenominator;
	OnValueCommitted = InArgs._OnValueCommitted;
	ChildSlot
	[
		SNew(SEditableTextBox)
		.Font(InArgs._Font)
		.SelectAllTextWhenFocused(true)
		.ClearKeyboardFocusOnCommit(false)
		.Text(this, &SFrameBasedTimeSignatureInput::GetValueAsText)
		.RevertTextOnEscape(true)
		.OnTextCommitted(this, &SFrameBasedTimeSignatureInput::TextBox_OnTextCommitted)
		.OnVerifyTextChanged(this, &SFrameBasedTimeSignatureInput::TextBox_VerifyTextChanged)
		.IsEnabled(InArgs._IsEnabled)
	];
}

FFrameBasedTimeSignature SFrameBasedTimeSignatureInput::GetValue() const
{
	return ValueAttribute.Get();
}

void SFrameBasedTimeSignatureInput::SetValue(const FFrameBasedTimeSignature& Value)
{
	ValueAttribute.Set(Value);
	OnValueCommitted.ExecuteIfBound(Value, ETextCommit::Default);
}


FString SFrameBasedTimeSignatureInput::GetValueAsString() const
{
	FFrameBasedTimeSignature CurrentValue = ValueAttribute.Get();
	return FString::Printf(TEXT("%d/%d"), CurrentValue.Numerator, CurrentValue.Denominator);
}

FText SFrameBasedTimeSignatureInput::GetValueAsText() const
{
	return FText::FromString(GetValueAsString());
}

TOptional<FFrameBasedTimeSignature> SFrameBasedTimeSignatureInput::ParseValueFromString(const FString& InString)
{
	TOptional<FFrameBasedTimeSignature> OutTimeSig;
	FRegexPattern TimeSigPattern(TEXT("^(\\d+)\\/(\\d+)$"));
	FRegexMatcher TimeSigMatcher(TimeSigPattern, InString.Replace(TEXT(" "), TEXT("")));
	if (TimeSigMatcher.FindNext())
	{
		OutTimeSig = FFrameBasedTimeSignature(
			FCString::Atoi(*TimeSigMatcher.GetCaptureGroup(1)),
			FCString::Atoi(*TimeSigMatcher.GetCaptureGroup(2)));
	}

	return OutTimeSig;
}

void SFrameBasedTimeSignatureInput::TextBox_OnTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
{
	TOptional<FFrameBasedTimeSignature> NewValue = ParseValueFromString(NewText.ToString());
	if (NewValue.IsSet())
	{
		if (!ValueAttribute.IsBound())
		{
			ValueAttribute.Set(NewValue.GetValue());
		}
		
		OnValueCommitted.ExecuteIfBound(NewValue.GetValue(), CommitInfo);
	}
}

bool SFrameBasedTimeSignatureInput::TextBox_VerifyTextChanged(const FText& NewText, FText& OutErrorMessage) const
{
	TOptional<FFrameBasedTimeSignature> NewValue = ParseValueFromString(NewText.ToString());
	if (NewValue.IsSet())
	{
		int32 MaxNum = FMath::Max(1, MaxNumerator.IsSet() ? MaxNumerator.Get() : MAX_int32);
		if (NewValue->Numerator <= 0 || NewValue->Numerator > MaxNum)
		{
			OutErrorMessage = FText::Format(LOCTEXT("InputError_Numerator", "Time signature numerator must be from 1 to {0}"), MaxNum);
			return false;
		}

		int32 MaxDen = FMath::Max(1, MaxDenominator.IsSet() ? MaxDenominator.Get() : MAX_int32);
		if (NewValue->Denominator <= 0 || NewValue->Denominator > MaxDen)
		{
			OutErrorMessage = FText::Format(LOCTEXT("InputError_Denominator", "Time signature denominator must be from 1 to {0}"), MaxNum);
			return false;
		}
		return true;
	}
	OutErrorMessage = LOCTEXT("InputError_TimeSignature", "Time signature must be in the form '4/4', '7/8', etc.");
	return false;
}

#undef LOCTEXT_NAMESPACE //SFrameBasedTimeSignatureInput