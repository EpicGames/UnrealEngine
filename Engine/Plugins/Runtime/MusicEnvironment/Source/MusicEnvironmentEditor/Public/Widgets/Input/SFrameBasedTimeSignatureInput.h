// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "FrameBasedMusicMap.h"
#include "Styling/CoreStyle.h"

#define UE_API MUSICENVIRONMENTEDITOR_API

class SFrameBasedTimeSignatureInput : public SCompoundWidget
{
public:

	/** Notification for numeric value committed */
	DECLARE_DELEGATE_TwoParams(FOnValueCommitted, const FFrameBasedTimeSignature&, ETextCommit::Type);

	SLATE_BEGIN_ARGS(SFrameBasedTimeSignatureInput)
		: _Value(FFrameBasedTimeSignature())
		, _MaxNumerator(DefaultMaxNumerator)
		, _MaxDenominator(DefaultMaxDenominator)
		, _IsEnabled(true)
		, _OnValueCommitted()
		, _Font(FCoreStyle::Get().GetFontStyle(TEXT("NormalFont")))
	{}

	SLATE_ATTRIBUTE(FFrameBasedTimeSignature, Value)
		
	SLATE_ATTRIBUTE(int32, MaxNumerator)

	SLATE_ATTRIBUTE(int32, MaxDenominator)

	SLATE_ATTRIBUTE(bool, IsEnabled)

	SLATE_EVENT(FOnValueCommitted, OnValueCommitted)
		
	SLATE_ATTRIBUTE(FSlateFontInfo, Font)
		
SLATE_END_ARGS()

UE_API void Construct(const FArguments& InArgs);

	UE_API FFrameBasedTimeSignature GetValue() const;

	UE_API void SetValue(const FFrameBasedTimeSignature& Value);
	
private:

	/** The default minimum fractional digits */
	static constexpr int32 DefaultMaxNumerator = 99;

	/** The default maximum fractional digits */
	static constexpr int32 DefaultMaxDenominator = 99;

	TAttribute<FFrameBasedTimeSignature> ValueAttribute;
	TAttribute< int32 > MaxNumerator;
	TAttribute< int32 > MaxDenominator;
	FOnValueCommitted OnValueCommitted;

	UE_API FString GetValueAsString() const;
	UE_API FText GetValueAsText() const;

	static UE_API TOptional<FFrameBasedTimeSignature> ParseValueFromString(const FString& InString); 

	UE_API void TextBox_OnTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo);
	
	UE_API bool TextBox_VerifyTextChanged(const FText& InText, FText& OutErrorMessage) const; 
};

#undef UE_API
