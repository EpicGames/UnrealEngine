// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "FrameBasedMusicMap.h"

#if 1 // WITH_DEV_AUTOMATION_TESTS

namespace MusicEnvironmentTests
{
	int32 BPMToMidiTempo(float Bpm)
	{
		return Bpm == 0.0 ? 0 : static_cast<int32>(60000000.0 / Bpm);
	}

	double TicksToMs(float Bpm, int32 DeltaTicks)
	{
		return static_cast<double>(DeltaTicks) * static_cast<double>(BPMToMidiTempo(Bpm)) / (static_cast<double>(MusicalTime::TicksPerQuarterNote) * 1000.0);
	}

	double TicksToSeconds(float Bpm, int32 DeltaTicks)
	{
		return TicksToMs(Bpm, DeltaTicks) / 1000.0;
	}

	int32 BarsToTick(const FFrameBasedTimeSignature& TimeSig, int32 DeltaBars)
	{
		int32 TicksPerBeat = MusicalTime::TicksPerQuarterNote * 4 / TimeSig.Denominator;
		int32 TicksPerBar = TicksPerBeat * TimeSig.Numerator;
		return DeltaBars * TicksPerBar;
	}
	
	BEGIN_DEFINE_SPEC(
		FFrameBasedMusicMapSpec,
		"MusicEnvironment.FrameBasedMusicMap",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	struct FScopeContext
	{
		FScopeContext(FAutomationTestBase& InTest, const FString& Context)
				: Test(InTest)
		{
			Test.PushContext(Context);
		}

		~FScopeContext()
		{
			Test.PopContext();
		}

		FScopeContext(const FScopeContext& Other) = delete;
		FScopeContext(FScopeContext&& Other) = delete;
		FScopeContext& operator=(const FScopeContext& Other) = delete;
		FScopeContext operator=(FScopeContext Other) = delete;
	private:
		FAutomationTestBase& Test;
	};
	
	bool TestMusicMapIsEmpty(UFrameBasedMusicMap* MusicMap)
	{
		FScopeContext ScopeContext(*this, TEXT("Test Music Map Is Empty"));
		bool Result = true;
		Result &= TestEqual(TEXT("MusicMap->FrameResolution"), MusicMap->FrameResolution, FFrameRate(0, 0));
		Result &= TestTrue(TEXT("MusicMap->TempoMap.IsEmpty()"), MusicMap->TempoMap.IsEmpty());
		Result &= TestTrue(TEXT("MusicMap->BarMap.IsEmpty()"), MusicMap->BarMap.IsEmpty());
		return Result;
	}

	bool TestMusicMapInitialized(UFrameBasedMusicMap* MusicMap, const FFrameRate& FrameResolution, float TempoBpm, const FFrameBasedTimeSignature& TimeSig)
	{
		FScopeContext ScopeContext(*this, TEXT("Test Music Map Initialized"));
		bool Result = true;
		Result &= TestEqual(TEXT("MusicMap->FrameResolution"), MusicMap->FrameResolution, FrameResolution);
		
		if (TestEqual(TEXT("MusicMap->TempoMap.Num()"), MusicMap->TempoMap.Num(), 1))
		{
			Result &= TestFrameBasedTempoPoint(TEXT("MusicMap->TempoMap[0]"), MusicMap->TempoMap[0], 0.0f, 0, TempoBpm);
		}
		else
		{
			Result = false;
		}

		if (TestEqual(TEXT("MusicMap->BarMap.Num()"), MusicMap->BarMap.Num(), 1))
		{
			Result &= TestFrameBasedTimeSignaturePoint(TEXT("MusicMap->BarMap[0]"), MusicMap->BarMap[0], 0.0, 0, 0, TimeSig);
		}
		else
		{
			Result = false;
		}
		
		return Result;
	}
	
	bool TestFrameBasedTempoPoint(const FString& What, const FFrameBasedTempoPoint& Point, float ExpectedMs, int32 ExpectedTick, float ExpectedTempoBpm)
	{
		bool Result = true;
		auto ExactlyWhat = [&What](const FString& PropertyName)
		{
			return FString::Format(TEXT("{0}.{1}"), { What, PropertyName });
		};
		
		int32 MicrosecondsPerQuarterNote = BPMToMidiTempo(ExpectedTempoBpm);
		
		Result &= TestEqual(ExactlyWhat("OnMs"), Point.OnMs, ExpectedMs, 0.01f);
		Result &= TestEqual(ExactlyWhat("OnTick"), Point.OnTick, ExpectedTick);
		Result &= TestEqual(ExactlyWhat("MicrosecondsPerQuarterNote"), Point.MicrosecondsPerQuarterNote, MicrosecondsPerQuarterNote);

		return Result;
	}

	bool TestMusicMapTempoPoints(const FString& What, const TArray<FFrameBasedTempoPoint>& ActualTempoPoints, const TArray<FFrameBasedTempoPoint>& ExpectedTempoPoints)
	{
		if (TestEqual(FString::Format(TEXT("{0}.{1}"), {What,  TEXT("Num()"})), ActualTempoPoints.Num(), ExpectedTempoPoints.Num()))
		{
			bool Result = true;
			for (int32 Index = 0; Index < ActualTempoPoints.Num(); ++Index)
			{
				float Ms = ExpectedTempoPoints[Index].OnMs;
				int32 Tick = ExpectedTempoPoints[Index].OnTick;
				float Bpm = ExpectedTempoPoints[Index].Bpm();
				Result &= TestFrameBasedTempoPoint(FString::Format(TEXT("{0}[{1}]"), {What, Index}), ActualTempoPoints[Index], Ms, Tick, Bpm);
			}
			return Result;
		}

		return false;
	}

	bool TestFrameBasedTimeSignaturePoint(const FString& What, const FFrameBasedTimeSignaturePoint& Point, double ExpectedFrame, int32 ExpectedTick, int32 ExpectedBar, const FFrameBasedTimeSignature& ExpectedTimeSig)
	{
		auto ExactlyWhat = [&What](const FString& PropertyName)
		{
			return FString::Format(TEXT("{0}.{1}"), { What, PropertyName });
		};
		bool Result = true;

		Result &= TestEqual(ExactlyWhat(TEXT("OnFrame")), Point.OnFrame, ExpectedFrame, 0.01);
		Result &= TestEqual(ExactlyWhat(TEXT("OnTick")), Point.OnTick, ExpectedTick);
		Result &= TestEqual(ExactlyWhat(TEXT("OnBar")), Point.OnBar, ExpectedBar);
		Result &= TestEqual(ExactlyWhat(TEXT("TimeSignature.Numerator")), Point.TimeSignature.Numerator, ExpectedTimeSig.Numerator);
		Result &= TestEqual(ExactlyWhat(TEXT("TimeSignature.Denominator")), Point.TimeSignature.Denominator, ExpectedTimeSig.Denominator);

		return Result;
	}

	bool TestMusicMapTimeSignaturePoints(const FString& What, const TArray<FFrameBasedTimeSignaturePoint>& ActualTimeSigPoints, const TArray<FFrameBasedTimeSignaturePoint>& ExpectedBarMap)
	{
		if (TestEqual(FString::Format(TEXT("{0}.{1}"), {What,  TEXT("Num()") }), ActualTimeSigPoints.Num(), ExpectedBarMap.Num()))
		{
			bool Result = true;
			for (int32 Index = 0; Index < ActualTimeSigPoints.Num(); ++Index)
			{
				double Frame = ExpectedBarMap[Index].OnFrame;
				int32 Tick = ExpectedBarMap[Index].OnTick;
				int32 Bar = ExpectedBarMap[Index].OnBar;
				FFrameBasedTimeSignature TimeSig = ExpectedBarMap[Index].TimeSignature;
				Result &= TestFrameBasedTimeSignaturePoint(FString::Format(TEXT("{0}[{1}]"), {What, Index}), ActualTimeSigPoints[Index], Frame, Tick, Bar, TimeSig);
			}
		}
		return false;
	}
	
	END_DEFINE_SPEC(FFrameBasedMusicMapSpec)

	void FFrameBasedMusicMapSpec::Define()
	{
		Describe(TEXT("Defaults"), [this]()
		{
			It(TEXT("Empty"), [this]()
			{
				UFrameBasedMusicMap* MusicMap = NewObject<UFrameBasedMusicMap>();
				TestMusicMapIsEmpty(MusicMap);
			});
			
			It(TEXT("Initialized"), [this]()
			{
				const FFrameRate FrameResolution(24000, 1);
				float TempoBpm = 120.0f;
				const FFrameBasedTimeSignature TimeSig(4, 4);
				
				UFrameBasedMusicMap* MusicMap = NewObject<UFrameBasedMusicMap>();
				MusicMap->SetFrameResolution(FrameResolution);
				MusicMap->Init(TempoBpm, TimeSig);

				TestMusicMapInitialized(MusicMap, FrameResolution, TempoBpm, TimeSig);
			});

			It(TEXT("AddTimeTimeSignature"), [this]()
			{
				const FFrameRate FrameResolution(24000, 1);
				float InitTempoBpm = 120.0f;
				const FFrameBasedTimeSignature InitTimeSig(4, 4);
							
				UFrameBasedMusicMap* MusicMap = NewObject<UFrameBasedMusicMap>();
				MusicMap->SetFrameResolution(FrameResolution);
				MusicMap->Init(InitTempoBpm, InitTimeSig);

				if (!TestMusicMapInitialized(MusicMap, FrameResolution, InitTempoBpm, InitTimeSig))
				{
					return;
				}

				int32 AtBar = 1;
				int32 AtTick = BarsToTick(InitTimeSig, AtBar);
				double AtFrame = FrameResolution.AsFrameTime(TicksToSeconds(InitTempoBpm, AtTick)).AsDecimal();
				FFrameBasedTimeSignature WithTimeSig(3, 4);
				
				MusicMap->AddTimeSignature(AtTick, AtBar, WithTimeSig.Numerator, WithTimeSig.Denominator);

				TArray<FFrameBasedTempoPoint> ExpectedTempoMap;
				ExpectedTempoMap.Add(FFrameBasedTempoPoint(InitTempoBpm, 0, 0.0f));

				TArray<FFrameBasedTimeSignaturePoint> ExpectedBarMap;
				ExpectedBarMap.Add(FFrameBasedTimeSignaturePoint(InitTimeSig, 0, 0, 0.0));
				ExpectedBarMap.Add(FFrameBasedTimeSignaturePoint(WithTimeSig, 1, AtTick, AtFrame));
				
				TestMusicMapTempoPoints(TEXT("MusicMap->TempoMap"), MusicMap->TempoMap, ExpectedTempoMap);
				TestMusicMapTimeSignaturePoints(TEXT("MusicMap->BarMap"), MusicMap->BarMap, ExpectedBarMap);
			});

			It(TEXT("AddTempo"), [this]()
			{
				const FFrameRate FrameResolution(24000, 1);
				float InitTempoBpm = 120.0f;
				const FFrameBasedTimeSignature InitTimeSig(4, 4);
										
				UFrameBasedMusicMap* MusicMap = NewObject<UFrameBasedMusicMap>();
				MusicMap->SetFrameResolution(FrameResolution);
				MusicMap->Init(InitTempoBpm, InitTimeSig);

				if (!TestMusicMapInitialized(MusicMap, FrameResolution, InitTempoBpm, InitTimeSig))
				{
					return;
				}

				int32 AtBar = 1;
				int32 AtTick = BarsToTick(InitTimeSig, AtBar);
				float AtMs = TicksToMs(InitTempoBpm, AtTick);
				float WithTempoBpm = 50.0f;
							
				MusicMap->AddTempo(AtTick, AtMs, WithTempoBpm);
				
				TArray<FFrameBasedTempoPoint> ExpectedTempoMap;
				ExpectedTempoMap.Add(FFrameBasedTempoPoint(InitTempoBpm, 0, 0.0f));
				ExpectedTempoMap.Add(FFrameBasedTempoPoint(WithTempoBpm, AtTick, AtMs));

				TArray<FFrameBasedTimeSignaturePoint> ExpectedBarMap;
				ExpectedBarMap.Add(FFrameBasedTimeSignaturePoint(InitTimeSig, 0, 0, 0.0));

				TestMusicMapTempoPoints(TEXT("MusicMap->TempoMap"), MusicMap->TempoMap, ExpectedTempoMap);
				TestMusicMapTimeSignaturePoints(TEXT("MusicMap->BarMap"), MusicMap->BarMap, ExpectedBarMap);
			});
			
			It(TEXT("InsertTimeSignature"), [this]()
			{
				const FFrameRate FrameResolution(24000, 1);
				float InitTempoBpm = 120.0f;
				const FFrameBasedTimeSignature InitTimeSig(4, 4);
												
				UFrameBasedMusicMap* MusicMap = NewObject<UFrameBasedMusicMap>();
				MusicMap->SetFrameResolution(FrameResolution);
				MusicMap->Init(InitTempoBpm, InitTimeSig);

				if (!TestMusicMapInitialized(MusicMap, FrameResolution, InitTempoBpm, InitTimeSig))
				{
					return;
				}
				
				int32 AtBar = 1;
				int32 AtTick = BarsToTick(InitTimeSig, AtBar);
				double AtFrame = FrameResolution.AsFrameTime(TicksToSeconds(InitTempoBpm, AtTick)).AsDecimal();
				FFrameBasedTimeSignature WithTimeSig(3, 4);
				
				MusicMap->InsertTimeSignature(AtBar, WithTimeSig.Numerator, WithTimeSig.Denominator);

				TArray<FFrameBasedTempoPoint> ExpectedTempoMap;
				ExpectedTempoMap.Add(FFrameBasedTempoPoint(InitTempoBpm, 0, 0.0f));

				TArray<FFrameBasedTimeSignaturePoint> ExpectedBarMap;
				ExpectedBarMap.Add(FFrameBasedTimeSignaturePoint(InitTimeSig, 0, 0, 0.0));
				ExpectedBarMap.Add(FFrameBasedTimeSignaturePoint(WithTimeSig, AtBar, AtTick, AtFrame));

				TestMusicMapTempoPoints(TEXT("MusicMap->TempoMap"), MusicMap->TempoMap, ExpectedTempoMap);
				TestMusicMapTimeSignaturePoints(TEXT("MusicMap->BarMap"), MusicMap->BarMap, ExpectedBarMap);
			});

			It(TEXT("InsertTempo"), [this]()
			{
				const FFrameRate FrameResolution(24000, 1);
				float InitTempoBpm = 120.0f;
				const FFrameBasedTimeSignature InitTimeSig(4, 4);
													
				UFrameBasedMusicMap* MusicMap = NewObject<UFrameBasedMusicMap>();
				MusicMap->SetFrameResolution(FrameResolution);
				MusicMap->Init(InitTempoBpm, InitTimeSig);

				if (!TestMusicMapInitialized(MusicMap, FrameResolution, InitTempoBpm, InitTimeSig))
				{
					return;
				}

				int32 AtBar = 1;
				int32 AtTick = BarsToTick(InitTimeSig, AtBar);
				float AtMs = TicksToMs(InitTempoBpm, AtTick);
				float WithTempoBpm = 50.0f;
										
				MusicMap->InsertTempo(AtTick, WithTempoBpm);
				
				TArray<FFrameBasedTempoPoint> ExpectedTempoMap;
				ExpectedTempoMap.Add(FFrameBasedTempoPoint(InitTempoBpm, 0, 0.0f));
				ExpectedTempoMap.Add(FFrameBasedTempoPoint(WithTempoBpm, AtTick, AtMs));

				TArray<FFrameBasedTimeSignaturePoint> ExpectedBarMap;
				ExpectedBarMap.Add(FFrameBasedTimeSignaturePoint(InitTimeSig, 0, 0, 0.0));

				TestMusicMapTempoPoints(TEXT("MusicMap->TempoMap"), MusicMap->TempoMap, ExpectedTempoMap);
				TestMusicMapTimeSignaturePoints(TEXT("MusicMap->BarMap"), MusicMap->BarMap, ExpectedBarMap);
			});
		});

		Describe(TEXT("WithTempoChanges"), [this]()
		{
			It(TEXT("InsertTempo"), [this]()
			{
				
				const FFrameRate FrameResolution(24000, 1);
				float InitTempoBpm = 120.0f;
				const FFrameBasedTimeSignature InitTimeSig(4, 4);
															
				UFrameBasedMusicMap* MusicMap = NewObject<UFrameBasedMusicMap>();
				MusicMap->SetFrameResolution(FrameResolution);
				MusicMap->Init(InitTempoBpm, InitTimeSig);

				if (!TestMusicMapInitialized(MusicMap, FrameResolution, InitTempoBpm, InitTimeSig))
				{
					return;
				}
				
							
				TArray<FFrameBasedTempoPoint> ExpectedTempoMap;
				TArray<FFrameBasedTimeSignaturePoint> ExpectedBarMap;
				
				ExpectedTempoMap.Add(FFrameBasedTempoPoint(InitTempoBpm, 0, 0.0f));
				ExpectedBarMap.Add(FFrameBasedTimeSignaturePoint(InitTimeSig, 0, 0, 0.0));

				// add tempo point for tempo changes
				{
					int32 AtBar = 2;
					int32 AtTick = BarsToTick(InitTimeSig, AtBar);
					float AtMs = TicksToMs(InitTempoBpm, AtTick);
					float WithTempoBpm = 100.0f;
												
					MusicMap->AddTempo(AtTick, AtMs, WithTempoBpm);
					ExpectedTempoMap.Add(FFrameBasedTempoPoint(WithTempoBpm, AtTick, AtMs));

					FScopeContext ScopeContext(*this, FString::Format(TEXT("Test Add First Tempo Change: {0}bpm Tick: {1}, Ms: {2}"),
						{WithTempoBpm, AtTick, AtMs}));
					TestMusicMapTempoPoints(TEXT("MusicMap->TempoMap"), MusicMap->TempoMap, ExpectedTempoMap);
					TestMusicMapTimeSignaturePoints(TEXT("MusicMap->BarMap"), MusicMap->BarMap, ExpectedBarMap);
				}

				{
					int32 AtBar = 1;
					int32 AtTick = BarsToTick(InitTimeSig, AtBar);
					float AtMs = TicksToMs(InitTempoBpm, AtTick);
					float WithTempoBpm = 110.0f;
					MusicMap->InsertTempo(AtTick, WithTempoBpm);
					ExpectedTempoMap.Insert(FFrameBasedTempoPoint(WithTempoBpm, AtTick, AtMs), 1);

					// we have to adjust the time of the later tempo point after inserting a tempo point before it
					ExpectedTempoMap.Last().OnMs = AtMs + TicksToMs(WithTempoBpm, ExpectedTempoMap.Last().OnTick - AtTick);

					FScopeContext ScopeContext(*this, FString::Format(TEXT("Test Insert New Tempo Change: {0}bpm @ Tick: {1}, Ms: {2}"),
						{WithTempoBpm, AtTick, AtMs}));
					TestMusicMapTempoPoints(TEXT("MusicMap->TempoMap"), MusicMap->TempoMap, ExpectedTempoMap);
					TestMusicMapTimeSignaturePoints(TEXT("MusicMap->BarMap"), MusicMap->BarMap, ExpectedBarMap);
				}
				
			});

			It(TEXT("InsertTimeSignature"), [this]()
			{

				const FFrameRate FrameResolution(24000, 1);
				float InitTempoBpm = 120.0f;
				
				const FFrameBasedTimeSignature InitTimeSig(4, 4);

				float SecondTempoBpm = 100.0f;
				int32 SecondTempoTick = BarsToTick(InitTimeSig, 1);
				
				UFrameBasedMusicMap* MusicMap = NewObject<UFrameBasedMusicMap>();
				MusicMap->SetFrameResolution(FrameResolution);
				MusicMap->Init(InitTempoBpm, InitTimeSig);

				if (!TestMusicMapInitialized(MusicMap, FrameResolution, InitTempoBpm, InitTimeSig))
				{
					return;
				}

				TArray<FFrameBasedTempoPoint> ExpectedTempoMap;
				TArray<FFrameBasedTimeSignaturePoint> ExpectedBarMap;
				ExpectedBarMap.Add(FFrameBasedTimeSignaturePoint(InitTimeSig, 0, 0, 0.0));
				ExpectedTempoMap.Add(FFrameBasedTempoPoint(InitTempoBpm, 0, 0.0f));
				
				// add tempo point for tempo changes
				{
					float AtMs = TicksToMs(InitTempoBpm, SecondTempoTick);
					MusicMap->AddTempo(SecondTempoTick, AtMs, SecondTempoBpm);
					ExpectedTempoMap.Add(FFrameBasedTempoPoint(SecondTempoBpm, SecondTempoTick, AtMs));

					FScopeContext ScopeContext(*this, FString::Format(TEXT("Test Add First Tempo Change: {0}bpm Tick: {1}, Ms: {2}"),
						{SecondTempoBpm, SecondTempoTick, AtMs}));
					TestMusicMapTempoPoints(TEXT("MusicMap->TempoMap"), MusicMap->TempoMap, ExpectedTempoMap);
					TestMusicMapTimeSignaturePoints(TEXT("MusicMap->BarMap"), MusicMap->BarMap, ExpectedBarMap);
				}

				// insert Time Signature Change
				{
					int32 AtBar = 2;
					int32 AtTick = BarsToTick(InitTimeSig, AtBar);
					double AtMs = TicksToMs(InitTempoBpm, SecondTempoTick) + TicksToMs(SecondTempoBpm,  AtTick - SecondTempoTick);
					FFrameTime FrameTime = FrameResolution.AsFrameTime(AtMs / 1000.0);
					FFrameBasedTimeSignature WithTimeSig(3, 4);
												
					MusicMap->InsertTimeSignature(AtBar, WithTimeSig.Numerator, WithTimeSig.Denominator);
					ExpectedBarMap.Add(FFrameBasedTimeSignaturePoint(WithTimeSig, AtBar, AtTick, FrameTime.AsDecimal()));

					FScopeContext ScopeContext(*this, FString::Format(TEXT("Test Insert New Time Signature Change: {0}/{1} @ Bar: {2}, Tick: {3}, Frame: {4} "),
						{WithTimeSig.Numerator, WithTimeSig.Denominator, AtBar, AtTick, FrameTime.AsDecimal()}));
					TestMusicMapTempoPoints(TEXT("MusicMap->TempoMap"), MusicMap->TempoMap, ExpectedTempoMap);
					TestMusicMapTimeSignaturePoints(TEXT("MusicMap->BarMap"), MusicMap->BarMap, ExpectedBarMap);
				}
			});
		});
		

		Describe(TEXT("WithTimeSignatureChanges"), [this]()
		{
			It(TEXT("InsertTempo"), [this]()
			{
				const FFrameRate FrameResolution(24000, 1);
				float InitTempoBpm = 120.0f;
				const FFrameBasedTimeSignature InitTimeSig(4, 4);
															
				UFrameBasedMusicMap* MusicMap = NewObject<UFrameBasedMusicMap>();
				MusicMap->SetFrameResolution(FrameResolution);
				MusicMap->Init(InitTempoBpm, InitTimeSig);

				if (!TestMusicMapInitialized(MusicMap, FrameResolution, InitTempoBpm, InitTimeSig))
				{
					return;
				}

				TArray<FFrameBasedTempoPoint> ExpectedTempoMap;
                TArray<FFrameBasedTimeSignaturePoint> ExpectedBarMap;
				ExpectedBarMap.Add(FFrameBasedTimeSignaturePoint(InitTimeSig, 0, 0, 0.0));
				ExpectedTempoMap.Add(FFrameBasedTempoPoint(InitTempoBpm, 0, 0.0f));
				
				// add time signature for time signature changes
				{
					int32 AtBar = 2;
					int32 AtTick = BarsToTick(InitTimeSig, AtBar);
					FFrameTime FrameTime = FrameResolution.AsFrameTime(TicksToSeconds(InitTempoBpm, AtTick));
					FFrameBasedTimeSignature WithTimeSig(3, 4);
												
					MusicMap->AddTimeSignature(AtTick, AtBar, WithTimeSig.Numerator, WithTimeSig.Denominator);
					ExpectedBarMap.Add(FFrameBasedTimeSignaturePoint(WithTimeSig, AtBar, AtTick, FrameTime.AsDecimal()));

					FScopeContext ScopeContext(*this, FString::Format(TEXT("Test Add First Time Signature Change: {0}/{1} @ Bar: {2}, Tick: {3}, Frame: {4}"),
						{WithTimeSig.Numerator, WithTimeSig.Denominator, AtBar, AtTick, FrameTime.AsDecimal()}));
					
					TestMusicMapTempoPoints(TEXT("MusicMap->TempoMap"), MusicMap->TempoMap, ExpectedTempoMap);
					TestMusicMapTimeSignaturePoints(TEXT("MusicMap->BarMap"), MusicMap->BarMap, ExpectedBarMap);
				}

				{
					int32 AtBar = 1;
					int32 AtTick = BarsToTick(InitTimeSig, AtBar);
					float AtMs = TicksToMs(InitTempoBpm, AtTick);
					float WithTempoBpm = 110.0f;
					MusicMap->InsertTempo(AtTick, WithTempoBpm);
					ExpectedTempoMap.Insert(FFrameBasedTempoPoint(WithTempoBpm, AtTick, AtMs), 1);

					// we also have to adjust the time of the late time signature point since the tempo changed before it
					// calculate the ms based on the tick of the time sig. change
					double NewMs = AtMs + TicksToMs(WithTempoBpm, ExpectedBarMap.Last().OnTick - AtTick);
					FFrameTime NewFrameTime = FrameResolution.AsFrameTime(NewMs / 1000.0);
					ExpectedBarMap.Last().OnFrame = NewFrameTime.AsDecimal();

					FScopeContext ScopeContext(*this, FString::Format(TEXT("Test Insert New Tempo Change: {0}bpm @ Tick: {1}, Ms: {2}"),
						{WithTempoBpm, AtTick, AtMs}));
					
					TestMusicMapTempoPoints(TEXT("MusicMap->TempoMap"), MusicMap->TempoMap, ExpectedTempoMap);
					TestMusicMapTimeSignaturePoints(TEXT("MusicMap->BarMap"), MusicMap->BarMap, ExpectedBarMap);
				}
			});

			It(TEXT("InsertTimeSignature"), [this]()
			{
				const FFrameRate FrameResolution(24000, 1);
				float InitTempoBpm = 120.0f;
				const FFrameBasedTimeSignature InitTimeSig(4, 4);
											
				UFrameBasedMusicMap* MusicMap = NewObject<UFrameBasedMusicMap>();
				MusicMap->SetFrameResolution(FrameResolution);
				MusicMap->Init(InitTempoBpm, InitTimeSig);

				if (!TestMusicMapInitialized(MusicMap, FrameResolution, InitTempoBpm, InitTimeSig))
				{
					return;
				}

				TArray<FFrameBasedTempoPoint> ExpectedTempoMap;
                TArray<FFrameBasedTimeSignaturePoint> ExpectedBarMap;
				ExpectedBarMap.Add(FFrameBasedTimeSignaturePoint(InitTimeSig, 0, 0, 0.0));
				ExpectedTempoMap.Add(FFrameBasedTempoPoint(InitTempoBpm, 0, 0.0f));

				// add time signature for time signature changes
				{
					int32 AtBar = 2;
					int32 AtTick = BarsToTick(InitTimeSig, AtBar);
					FFrameTime FrameTime = FrameResolution.AsFrameTime(TicksToSeconds(InitTempoBpm, AtTick));
					FFrameBasedTimeSignature WithTimeSig(3, 4);
								
					MusicMap->AddTimeSignature(AtTick, AtBar, WithTimeSig.Numerator, WithTimeSig.Denominator);
					ExpectedBarMap.Add(FFrameBasedTimeSignaturePoint(WithTimeSig, AtBar, AtTick, FrameTime.AsDecimal()));

					FScopeContext ScopeContext(*this, FString::Format(TEXT("Test Add Time First Signature Change: {0}/{1} @ Bar: {2}, Tick: {3}, Frame: {4}"),
						{WithTimeSig.Numerator, WithTimeSig.Denominator, AtBar, AtTick, FrameTime.AsDecimal()}));
					TestMusicMapTempoPoints(TEXT("MusicMap->TempoMap"), MusicMap->TempoMap, ExpectedTempoMap);
					TestMusicMapTimeSignaturePoints(TEXT("MusicMap->BarMap"), MusicMap->BarMap, ExpectedBarMap);
				}

				// insert Time Signature Change
				{
					int32 AtBar = 1;
					int32 AtTick = BarsToTick(InitTimeSig, AtBar);
					double AtMs = TicksToMs(InitTempoBpm, AtTick);
					FFrameTime FrameTime = FrameResolution.AsFrameTime(AtMs / 1000.0);
					FFrameBasedTimeSignature WithTimeSig(2, 4);

					// changing the time signature will cause the bar mapping to change which means we need to update the later time signature position
					MusicMap->InsertTimeSignature(AtBar, WithTimeSig.Numerator, WithTimeSig.Denominator);
					ExpectedBarMap.Insert(FFrameBasedTimeSignaturePoint(WithTimeSig, AtBar, AtTick, FrameTime.AsDecimal()), 1);
					
					int32 NewTick = AtTick + BarsToTick(WithTimeSig, 1);
					double NewMs = AtMs + TicksToMs(InitTempoBpm, NewTick - AtTick);
					FFrameTime NewFrameTime = FrameResolution.AsFrameTime(NewMs / 1000.0);
					ExpectedBarMap.Last().OnTick = NewTick;
					ExpectedBarMap.Last().OnFrame = NewFrameTime.AsDecimal();

					FScopeContext ScopeContext(*this, FString::Format(TEXT("Test Insert New Time Signature Change: {0}/{1} @ Bar: {2}, Tick: {3}, Frame: {4} "),
						{WithTimeSig.Numerator, WithTimeSig.Denominator, AtBar, AtTick, FrameTime.AsDecimal()}));
					TestMusicMapTempoPoints(TEXT("MusicMap->TempoMap"), MusicMap->TempoMap, ExpectedTempoMap);
					TestMusicMapTimeSignaturePoints(TEXT("MusicMap->BarMap"), MusicMap->BarMap, ExpectedBarMap);
				}
				
				// insert Time Signature Change
				{
					int32 AtBar = 3;
					int32 AtTick = BarsToTick(InitTimeSig, 1) + BarsToTick({2, 4}, 1) + BarsToTick({3, 4}, 1);
					AddInfo(FString::Format(TEXT("{0} + {1} + {2} = {3}"), {BarsToTick(InitTimeSig, 1), BarsToTick({2, 4}, 1), BarsToTick({3, 4}, 1), AtTick}));
					double AtMs = TicksToMs(InitTempoBpm, AtTick);
					FFrameTime FrameTime = FrameResolution.AsFrameTime(AtMs / 1000.0);
					FFrameBasedTimeSignature WithTimeSig(6, 8);

					// changing the time signature will cause the bar mapping to change which means we need to update the later time signature position
					MusicMap->InsertTimeSignature(AtBar, WithTimeSig.Numerator, WithTimeSig.Denominator);
					ExpectedBarMap.Add(FFrameBasedTimeSignaturePoint(WithTimeSig, AtBar, AtTick, FrameTime.AsDecimal()));
					
					FScopeContext ScopeContext(*this, FString::Format(TEXT("Test Insert New Time Signature Change: {0}/{1} @ Bar: {2}, Tick: {3}, Frame: {4} "),
						{WithTimeSig.Numerator, WithTimeSig.Denominator, AtBar, AtTick, FrameTime.AsDecimal()}));
					TestMusicMapTempoPoints(TEXT("MusicMap->TempoMap"), MusicMap->TempoMap, ExpectedTempoMap);
					TestMusicMapTimeSignaturePoints(TEXT("MusicMap->BarMap"), MusicMap->BarMap, ExpectedBarMap);
				}
			});
		});
	}
};

#endif // WITH_DEV_AUTOMATION_TESTS